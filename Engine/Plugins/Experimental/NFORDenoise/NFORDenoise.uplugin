{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NFORDenoise",
	"Description": "Spatial-temporal denoising engine for the Unreal Path Tracer (mainly used with MRQ). It denoises each pixel based on the surrounding patches in space and time in all directions. The algorithm is mainly inspired by Nonlinearly Weighted First-order Regression (NFOR) for Denoising Monte Carlo Renderings.",
	"Category": "Denoising",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux"
	],
	"Modules": [
		{
			"Name": "NFORDenoise",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	]
}
