// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "NFORRegressionCommon.ush"

#ifndef SOURCE_CHANNEL_COUNT
#define SOURCE_CHANNEL_COUNT 4
#endif

#if !COMPUTESHADER
#define THREAD_GROUP_SIZE 1
#endif

#if SOURCE_CHANNEL_COUNT == 1
#define TPixelValue float
#elif SOURCE_CHANNEL_COUNT == 2
#define TPixelValue float2
#elif SOURCE_CHANNEL_COUNT == 3
#define TPixelValue float3
#elif SOURCE_CHANNEL_COUNT == 4
#define TPixelValue float4
#endif

#define IMAGE_VARINCE_NORMAL		0
#define IMAGE_VARIANCE_GREYSCALE	1
#define IMAGE_VARIANCE_COLORED		2

#ifndef IMAGE_VARIANCE_TYPE
#define IMAGE_VARIANCE_TYPE IMAGE_VARIANCE_GREYSCALE
#endif

#define PRE_ALBEDO_DIVIDE_DISABLED	0
#define PRE_ALBEDO_DIVIDE_EACH		1
#define PRE_ALBEDO_DIVIDE_FINAL		2

#ifndef PRE_ALBEDO_DIVIDE
#define PRE_ALBEDO_DIVIDE PRE_ALBEDO_DIVIDE_DISABLED
#endif

#ifndef APPEND_CONSTANT_DIMENSION_TO_X
#define APPEND_CONSTANT_DIMENSION_TO_X 0
#endif

#ifndef NONLOCALMEAN_SEPARATE_SOURCE
#define NONLOCALMEAN_SEPARATE_SOURCE 0
#endif

#define NONLOCALMEAN_PATCHONLY		0
#define NONLOCALMEAN_DISTACNE_PATCH 1
#define NONLOCALMEAN_WEIGHT_TYPE	NONLOCALMEAN_PATCHONLY

#define NONLOCALMEAN_SEPERABLE_FILTER_HORIZONTAL	0
#define NONLOCALMEAN_SEPERABLE_FILTER_VERTICAL		1

#ifndef NONLOCALMEAN_SEPRERABLE_PASS
#define NONLOCALMEAN_SEPRERABLE_PASS NONLOCALMEAN_SEPERABLE_FILTER_HORIZONTAL
#endif

#define NONLOCALMEAN_ATLAS_ONE_SYMMETRIC_PAIR	0
#define NONLOCALMEAN_ATLAS_TWO_SYMMETRIC_PAIR	1

#ifndef NONLOCALMEAN_ATLAS_TYPE
#define NONLOCALMEAN_ATLAS_TYPE NONLOCALMEAN_ATLAS_TWO_SYMMETRIC_PAIR
#endif

#if NONLOCALMEAN_ATLAS_TYPE == NONLOCALMEAN_ATLAS_ONE_SYMMETRIC_PAIR
#define TAtlasValueType float2
#elif NONLOCALMEAN_ATLAS_TYPE == NONLOCALMEAN_ATLAS_TWO_SYMMETRIC_PAIR
#define TAtlasValueType float4
#else
#error NONLOCALMEAN_ATLAS_TYPE is not supported.
#endif

#ifndef BUFFER_PASS_THROUGH
#define BUFFER_PASS_THROUGH 0
#endif

#define NONLOCALMEAN_ATLAS_SYMMETRIC_PAIR_COUNT (NONLOCALMEAN_ATLAS_TYPE+1)

#define NLM_WEIGHTLAYOUT_NONE						  0
#define NLM_WEIGHTLAYOUT_NUM_OF_WEIGHTS_PER_PIXELxWxH 1
#define NLM_WEIGHTLAYOUT_WxHxNUM_OF_WEIGHTS_PER_PIXEL 2
#define NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4	  3

#ifndef NLM_WEIGHTLAYOUT
#define NLM_WEIGHTLAYOUT NLM_WEIGHTLAYOUT_NONE
#endif

#if NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4
#define TWeightReadType float4
#define WEIGHT_PIXEL_INCREMENT	4
#else
#define TWeightReadType float
#define WEIGHT_PIXEL_INCREMENT	1
#endif

#ifndef NUM_FEATURE
#define NUM_FEATURE 0
#endif

#ifndef SMALL_MATRIX_OPTIMIZE
#define SMALL_MATRIX_OPTIMIZE 0
#endif

// Sampling steps for performance optimization.
#ifndef USE_SAMPLING_STEP
#define USE_SAMPLING_STEP 0
#endif

#define SAMPLING_TYPE_INCREMENTAL			0
#define SAMPLING_TYPE_FOCUS_CURRENT_FRAME	1

#define SAMPLING_TYPE SAMPLING_TYPE_FOCUS_CURRENT_FRAME

// Matrix multiplication
#define WEIGHTED_MULTIPLICATION_QUADRATIC	0	//X^TWX
#define WEIGHTED_MULTIPLICATION_GENERALIZED	1	//X^TWY

#ifndef WEIGHTED_MULTIPLICATION_TYPE
#define WEIGHTED_MULTIPLICATION_TYPE WEIGHTED_MULTIPLICATION_QUADRATIC
#endif

// Linear equation system solver
#define INPUT_MATRIX_TYPE_SUCCESS	0
#define INPUT_MATRIX_TYPE_FAIL		1

#ifndef INPUT_MATRIX_TYPE
#define INPUT_MATRIX_TYPE	INPUT_MATRIX_TYPE_SUCCESS
#endif

#define OUTPUT_MATRIX_TYPE_SUCCESS	0
#define OUTPUT_MATRIX_TYPE_FAIL		1

#ifndef OUTPUT_INDICES
#define OUTPUT_INDICES 0
#endif

// Reconstruction
#define RECONSTRUCTION_TYPE_SCATTER 0
#define RECONSTRUCTION_TYPE_GATHER	1

#ifndef RECONSTRUCTION_TYPE
#define RECONSTRUCTION_TYPE RECONSTRUCTION_TYPE_SCATTER
#endif

#define RADIANCE_PREPROCESS_SCALE_FACTOR 0.1f
#define RADIANCE_POSTPROCESS_INVERSE_SCALE_FACTOR (1/RADIANCE_PREPROCESS_SCALE_FACTOR);

//--------------------------------------------------------------------------------------------------
// Util and general texture operations
//--------------------------------------------------------------------------------------------------
#define TEXTURE_OPS_MULTIPLY		0
#define TEXTURE_OPS_DIVIDE			1
#define TEXTURE_OPS_ADD_CONSTANT	2
#define TEXTURE_OPS_ACCUMULATE		3

#ifndef TEXTURE_OPS
#define TEXTURE_OPS TEXTURE_OPS_MULTIPLY
#endif

#define TEXTURE_COPY_TARGET_SINGLE_CHANNEL	0
#define TEXTURE_COPY_SOURCE_SINGLE_CHANNEL	1

#ifndef TEXTURE_COPY_TYPE
#define TEXTURE_COPY_TYPE TEXTURE_COPY_TARGET_SINGLE_CHANNEL
#endif

#ifndef ACCUMULATE_BY_MASK
#define ACCUMULATE_BY_MASK 0
#endif

float Length2(TPixelValue Value)
{
#if SOURCE_CHANNEL_COUNT == 1
	return Value * Value;
#elif SOURCE_CHANNEL_COUNT == 4
	return length2(Value.rgb);
#else
	return length2(Value);
#endif
}

TPixelValue GetImageValue(int2 P, Texture2D<TPixelValue> inImage)
{
	return inImage.Load(int3(P, 0));
}

int2 GetMirroredPosition(int2 P, int2 inTextureSize)
{
	int2 TextureSizeMax = inTextureSize - 1;
	P = abs(TextureSizeMax - abs(P - TextureSizeMax));
	return P;
}

//--------------------------------------------------------------------------------------------------
// General texture operations
Texture2D<TPixelValue> Source;
Texture2D<uint> Mask;
RWTexture2D<float4> RWTarget;
int2 SourcePosition;
int2 TargetPosition;
int2 Size;
int ForceOperation;
float4 ConstantValue;

float4 LoadSource(uint2 Position)
{
	float4 SourceValue = 0;

#if SOURCE_CHANNEL_COUNT == 1
	SourceValue.x = Source.Load(uint3(Position, 0));
#elif SOURCE_CHANNEL_COUNT == 2
	SourceValue.xy = Source.Load(uint3(Position, 0));
#elif SOURCE_CHANNEL_COUNT == 3
	SourceValue.xyz = Source.Load(uint3(Position, 0));
#elif SOURCE_CHANNEL_COUNT == 4
	SourceValue = Source.Load(uint3(Position, 0));
#endif
	return SourceValue;
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void TextureOperationCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= (uint2)Size))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;
	const uint2 ResolvedSourcePosition = max(SourcePosition + Position, 0);
	const uint2 ResolvedTargetPosition = TargetPosition + Position;

	if (all(ResolvedTargetPosition >= 0))
	{
#if TEXTURE_OPS == TEXTURE_OPS_MULTIPLY
		float4 STexel = LoadSource(ResolvedSourcePosition);
		RWTarget[ResolvedTargetPosition].rgb *= lerp(1.0f, STexel.rgb, (STexel.rgb != 0) | ForceOperation);
#elif TEXTURE_OPS == TEXTURE_OPS_DIVIDE
		float4 STexel = LoadSource(ResolvedSourcePosition);
		RWTarget[ResolvedTargetPosition].rgb /= lerp(1.0f, STexel.rgb, (STexel.rgb != 0) | ForceOperation);
#elif TEXTURE_OPS == TEXTURE_OPS_ADD_CONSTANT
		float3 Value = 0;
#if ACCUMULATE_BY_MASK
		uint MaskId = clamp(Mask[ResolvedTargetPosition], 0, 3); // support up to 4 masked add.
		Value = ConstantValue[MaskId];
#else
		Value = ConstantValue.rgb;
#endif
		RWTarget[ResolvedTargetPosition].rgb += Value;
#elif TEXTURE_OPS == TEXTURE_OPS_ACCUMULATE
		float4 STexel = LoadSource(ResolvedSourcePosition);
		RWTarget[ResolvedTargetPosition] += STexel;
#endif
	}
}

int2 SourceOffset;
int2 TargetOffset;
int Channel;
int2 CopySize;
int2 TextureSize;

TPixelValue CopyTexturePS(float4 SvPosition : SV_POSITION) : SV_Target0
{
	uint2 Position = (uint2)SvPosition.xy + SourceOffset;
	return Source.Load(uint3(GetMirroredPosition(Position, TextureSize), 0));
}

#if TEXTURE_COPY_TYPE == TEXTURE_COPY_TARGET_SINGLE_CHANNEL
Texture2D<float4> CopySource;
RWTexture2D<float> RWCopyTarget;
#elif TEXTURE_COPY_TYPE == TEXTURE_COPY_SOURCE_SINGLE_CHANNEL
Texture2D<float> CopySource;
RWTexture2D<float4> RWCopyTarget;
#endif

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void CopyTextureSingleChannelCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= (uint2)CopySize))
	{
		return;
	}

	uint2 ResolvedSourcePosition = DispatchThreadID.xy + SourceOffset;
	uint2 ResolvedTargetPosition = DispatchThreadID.xy + TargetOffset;

	if (all(ResolvedTargetPosition >= 0))
	{
#if TEXTURE_COPY_TYPE == TEXTURE_COPY_TARGET_SINGLE_CHANNEL
		RWCopyTarget[ResolvedTargetPosition] = CopySource.Load(uint3(GetMirroredPosition(ResolvedSourcePosition, TextureSize), 0))[Channel];
#elif TEXTURE_COPY_TYPE == TEXTURE_COPY_SOURCE_SINGLE_CHANNEL
		RWCopyTarget[ResolvedTargetPosition][Channel] = CopySource.Load(uint3(GetMirroredPosition(ResolvedSourcePosition, TextureSize), 0));
#endif
	}
}

//--------------------------------------------------------------------------------------------------
// Variance definition and functions
//
// Assumption: Variance texture is of type float4 and holds std instead
//
#define TImageVariance float4

float GetImageVariance(int2 P, Texture2D inVariance, int inVarianceChannelOffset)
{
	float Var = 0;
#if (IMAGE_VARIANCE_TYPE == IMAGE_VARIANCE_NORMAL) | (IMAGE_VARIANCE_TYPE == IMAGE_VARIANCE_GREYSCALE)
	// The loaded variance is actually the std
	float Std = inVariance.Load(int3(P, 0))[inVarianceChannelOffset];
	Var = Pow2(Std);	
#elif IMAGE_VARIANCE_TYPE == IMAGE_VARIANCE_COLORED
	// Assume channel independence and use Luminance() for grey scale
	// calculation.
	const float3 Constants = float3(0.09, 0.3481, 0.0121);
	float3 StdRGB = inVariance.Load(int3(P, 0)).rgb;
	Var = dot(Pow2(StdRGB), Constants);
#else
#error IMAGE_VARIANCE_TYPE not supported
#endif
	return Var;
}

//------------------------------------------------------------------------------------------------------
// Radiance normalization
//------------------------------------------------------------------------------------------------------

Texture2D<float4> Normal;
Texture2D<float4> NormalVariance;
RWTexture2D<uint> RWMask;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void ClassifyPreAlbedoDivideMaskIdCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= (uint2)TextureSize))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;
	float4 NormalValues = 0;
	NormalValues.rgb = Normal.Load(uint3(Position, 0)).rgb;
	NormalValues.a = NormalVariance.Load(uint3(Position, 0)).b;

	uint MaskId = 0;

	if (all(NormalValues == 0))
	{
		MaskId = 1;
	}

	RWMask[Position] = MaskId;
}

Texture2D<float4> Albedo;
RWTexture2D<float4> RWRadianceVariance;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void NormalizeRadianceVarianceByAlbedoCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= (uint2)Size))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;
	float AlbedoLum = Luminance(Albedo.Load(uint3(Position, 0)).rgb);

	// Since r stores the std instead of variance, need to divide albedo instead of albedo^2.
	RWRadianceVariance[Position].r /= lerp(1.0f, AlbedoLum, AlbedoLum > 0);
}

RWTexture2D<float4> RWImage;
RWTexture2D<float4> RWImageVariance;
float MaxValue;
int VarianceChannelOffset;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void AdjustFeatureRangeCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= (uint2)Size))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;

	float3 Feature = RWImage[Position].xyz;
#if (IMAGE_VARIANCE_TYPE == IMAGE_VARIANCE_NORMAL)
	float Value = length(Feature);
#elif (IMAGE_VARIANCE_TYPE == IMAGE_VARIANCE_GREYSCALE)
	float Value = Luminance(Feature);
#else
	float Value = -1.0f;
#endif

	const float ScaleFactor = lerp(1.0f, MaxValue / Value, Value > MaxValue);
	
	BRANCH
	if (ScaleFactor < 1.0f)
	{
		RWImage[Position].xyz *= ScaleFactor;
		RWImageVariance[Position][VarianceChannelOffset] *= ScaleFactor; // Scaling the std does not require power.
	}
}

//--------------------------------------------------------------------------------------------------
// Non-local mean definition and function
//--------------------------------------------------------------------------------------------------
// When NONLOCALMEAN_SEPARATE_SOURCE == 1, the NLM calculates the distance from source image to target image
// instead of on its own.

int PatchSize;
int PatchDistance;
float Bandwidth;

// Dispatch parameters
int DispatchId;
int2 DispatchTileSize;
int DispatchTileCount;
int4 SeparableFilteringRegion;
int3 DispatchRegionSize;

Texture2D<TPixelValue> Image;
Texture2D<TImageVariance> Variance;

#if NONLOCALMEAN_SEPARATE_SOURCE
Texture2D<TPixelValue> TargetImage;
Texture2D<TImageVariance> TargetVariance;
#else
#define TargetImage Image
#define TargetVariance Variance
#endif

struct FImageContext
{
	TPixelValue Value;
	float Variance;
};

FImageContext GetImageContext(int2 Position, Texture2D<TPixelValue> inImage, Texture2D inVariance, float VarianceScale = 1.0f)
{
	FImageContext ImageContext = (FImageContext)0;

	// Use mirrored position. Clamp to boarder has artifacts at border.
	Position = GetMirroredPosition(Position, TextureSize);

	ImageContext.Value = GetImageValue(Position, inImage);
	ImageContext.Variance = GetImageVariance(Position, inVariance, VarianceChannelOffset) * VarianceScale;

	return ImageContext;
}

// Note that GetSquaredDistance(P,Q) != GetSquaredDistance(Q,P), So we cannot share the weight with this asymetric distance.
float GetSquaredDistance(TPixelValue Cp, float VarP, TPixelValue Cq, float VarQ)
{
	const float Epsilon = 1e-8f; // Small enough to preserve dark areas

	float dpq = (Length2(Cp - Cq) - (VarP + min(VarP, VarQ))) / (Epsilon + Pow2(Bandwidth) * (VarP + VarQ));

	const float MaxDistance = 10000;
	dpq = min(dpq, MaxDistance);

	return dpq;
}

// Get the both way x=GetSquaredDistance(P,Q) and y=GetSquaredDistance(Q,P) together.
float2 GetSymmetricSquaredDistance(TPixelValue Cp, float VarP, TPixelValue Cq, float VarQ)
{
	const float Epsilon = 1e-8f; // Small enough to preserve dark areas

	float VarPQMin = min(VarP, VarQ);
	float2 VarPQ = float2((VarP + VarPQMin), (VarQ + VarPQMin));

	float2 dpq = (Length2(Cp - Cq) - VarPQ) / (Epsilon + Pow2(Bandwidth) * (VarP + VarQ));

	const float MaxDistance = 10000;
	dpq = min(dpq, MaxDistance);

	return dpq;
}

float GetSquaredDistance(FImageContext PCtx, FImageContext QCtx)
{
	return GetSquaredDistance(PCtx.Value, PCtx.Variance, QCtx.Value, QCtx.Variance);
}

float GetSquaredDistance(int2 P, int2 Q, float VarianceScale = 1.0f)
{
	FImageContext PCtx = GetImageContext(P, Image, Variance, VarianceScale);
	FImageContext QCtx = GetImageContext(Q, TargetImage, TargetVariance, VarianceScale);
	float dpq = GetSquaredDistance(PCtx, QCtx);
	return dpq;
}

float2 GetSymmetricSquaredDistance(FImageContext PCtx, FImageContext QCtx)
{
	return GetSymmetricSquaredDistance(PCtx.Value, PCtx.Variance, QCtx.Value, QCtx.Variance);
}

float2 GetSymmetricSquaredDistance(int2 P, int2 Q, float VarianceScale = 1.0f)
{
	FImageContext PCtx = GetImageContext(P, Image, Variance, VarianceScale);
	FImageContext QCtx = GetImageContext(Q, TargetImage, TargetVariance, VarianceScale);
	float2 dpq = GetSymmetricSquaredDistance(PCtx, QCtx);
	return dpq;
}

float GetPatchSquaredDistance(int2 P, int2 Q)
{
	const float VarianceScale = 1.0f;

	// TODO: z-order for performance improvement?
	float Sum = 0;
	for (int y = -PatchSize; y <= PatchSize; ++y)
	{
		for (int x = -PatchSize; x <= PatchSize; ++x)
		{
			int2 Pij = P + int2(x, y);
			int2 Qij = Q + int2(x, y);
			Sum += GetSquaredDistance(Pij, Qij, VarianceScale);
		}
	}

	// ||Pp - Pq||^2
	float PathSquareDistance = max(0, Sum / Pow2(2 * PatchSize + 1));
	return PathSquareDistance;
}

TAtlasValueType GetNonLocalMeanWeight(int2 P, int2 Q, TAtlasValueType PatchPQSqaureDistance)
{
	TAtlasValueType NonLocalMeanWeight = (TAtlasValueType)0;
#if NONLOCALMEAN_WEIGHT_TYPE == NONLOCALMEAN_PATCHONLY
	NonLocalMeanWeight = FastExp(-PatchPQSqaureDistance);
#elif NONLOCALMEAN_WEIGHT_TYPE == NONLOCALMEAN_DISTACNE_PATCH
	#error not implemented
	const float SigmaS = 2.0f;//TODO: move to CVar
	NonLocalMeanWeight = FastExp(-length2(P - Q) / (2 * Pow2(SigmaS))) * FastExp(-PatchPQSqaureDistance);
#endif

#if PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_DISABLED
	const TAtlasValueType kMinFloat16 = (TAtlasValueType)0.000000059604645f;
	//without albedo divide, below is better when scattering results.
	NonLocalMeanWeight = max(kMinFloat16, NonLocalMeanWeight);
#endif
	return NonLocalMeanWeight;
}

// P is on source image, Q is on target image
float GetNonLocalMeanWeight(int2 P, int2 Q)
{
	//TODO: add a box filter?
	float PatchpqSqaure = GetPatchSquaredDistance(P,Q);

	float NonLocalMeanWeight = 1.0f;
#if NONLOCALMEAN_WEIGHT_TYPE == NONLOCALMEAN_PATCHONLY
	NonLocalMeanWeight = FastExp(-PatchpqSqaure);
#elif NONLOCALMEAN_WEIGHT_TYPE == NONLOCALMEAN_DISTACNE_PATCH
	const float SigmaS = 2.0f;//TODO: move to CVar
	NonLocalMeanWeight = FastExp(-length2(P - Q) / (2 * Pow2(SigmaS))) * FastExp(-PatchpqSqaure);
#endif

#if PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_DISABLED
	const float kMinFloat16 = 0.000000059604645f;
	//without albedo divide, below is better when scattering results.
	NonLocalMeanWeight = max(kMinFloat16, NonLocalMeanWeight);
#endif
	return NonLocalMeanWeight;
}

RWTexture2D<TPixelValue> DenoisedImage;
Buffer<TWeightReadType> NonLocalMeanWeights;
int DenoisingChannelCount;
int4 FilteringRegion;

uint GetWeightBufferIndex(uint2 LocalPositionInRegion, uint WeightIndex, uint2 RegionSize)
{
#if (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NONE) \
	|| (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NUM_OF_WEIGHTS_PER_PIXELxWxH)
	const uint PatchWidth = PatchDistance * 2 + 1;
	const uint NumOfWeightsPerPixel = Pow2(PatchWidth);
	uint BufferIndex = WeightIndex + NumOfWeightsPerPixel * (LocalPositionInRegion.x + RegionSize.x * LocalPositionInRegion.y);
#elif NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_WxHxNUM_OF_WEIGHTS_PER_PIXEL
	// Best if neighbour pixels query the same weight index.
	uint BufferIndex = LocalPositionInRegion.x + RegionSize.x * (LocalPositionInRegion.y + RegionSize.y * WeightIndex);
#elif NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4
	// Assume the buffer layout is 4xWxHx[N/4] and the buffer is accessed with Buffer<float>
	uint StartIndex = WeightIndex / 4;
	uint WeightOffset = WeightIndex % 4;
	uint BufferIndex = WeightOffset + 4 *(LocalPositionInRegion.x + RegionSize.x * (LocalPositionInRegion.y + RegionSize.y * StartIndex));
#else
	#error not implemented
#endif
	return BufferIndex;
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void NonLocalMeanFilteringCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	int2 RegionSize = int2(FilteringRegion.z - FilteringRegion.x, FilteringRegion.w - FilteringRegion.y);
	if (any(DispatchThreadID.xy >= RegionSize))
	{
		return;
	}

	const uint PatchWidth = PatchDistance * 2 + 1;
	const uint NumOfWeightsPerPixel = Pow2(PatchWidth);
	const uint2 Position = DispatchThreadID.xy + FilteringRegion.xy;
	float TotalWeights = 0;
	TPixelValue Result = 0;

	for (int i = 0; i < NumOfWeightsPerPixel; i += WEIGHT_PIXEL_INCREMENT)
	{
		int x = i % PatchWidth - PatchDistance;
		int y = i / PatchWidth - PatchDistance;
		int2 Q = Position + int2(x,y);
#if NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NONE
		TWeightReadType Weight = GetNonLocalMeanWeight(Position, Q);
#else
		uint WeightIndex = i;// (x + PatchDistance) + (y + PatchDistance) * PatchWidth;
		uint BufferIndex = GetWeightBufferIndex(DispatchThreadID.xy, WeightIndex, RegionSize) / WEIGHT_PIXEL_INCREMENT;
		TWeightReadType Weight = NonLocalMeanWeights[BufferIndex];
#endif

		UNROLL
		for (int j = 0; j < WEIGHT_PIXEL_INCREMENT; ++j)
		{	
#if WEIGHT_PIXEL_INCREMENT == 1
			float LocalWeight = Weight;
#else
			int CombinedIndex = i + j; 
			float LocalWeight = Weight[j];
			x = CombinedIndex % PatchWidth - PatchDistance;
			y = CombinedIndex / PatchWidth - PatchDistance;
			Q = Position + int2(x,y);
			if (CombinedIndex < NumOfWeightsPerPixel)
#endif
			{
				Result += LocalWeight * GetImageValue(GetMirroredPosition(Q, TextureSize), TargetImage);
				TotalWeights += LocalWeight;
			}
		}
	}

	Result = Result / max(1e-6f, TotalWeights);

#if SOURCE_CHANNEL_COUNT == 1
	DenoisedImage[Position] = Result;
#else
	TPixelValue TargetValue = GetImageValue(GetMirroredPosition(Position, TextureSize), TargetImage);
	for (int i = 0; i < DenoisingChannelCount; ++i)
	{
		TargetValue[i] = Result[i];
	}
	DenoisedImage[Position] = TargetValue;
#endif
}

// Calculate the non-local mean weights for Region
int4 Region;
RWBuffer<float> RWNonLocalMeanWeights;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void NonLocalMeanWeightsCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	int2 RegionSize = int2(Region.z - Region.x, Region.w - Region.y);
	if (any(DispatchThreadID.xy >= RegionSize))
	{
		return;
	}
	const uint2 Position = DispatchThreadID.xy + Region.xy;
	const int PatchDiameter = 2 * PatchDistance + 1;
	const int PixelOffset = Pow2(PatchDiameter);
	const int BufferOffset = (DispatchThreadID.x + DispatchThreadID.y * RegionSize.x) * PixelOffset;

	for (int i = 0; i < PixelOffset; ++i)
	{
		int2 Offset = int2(i % PatchDiameter, i / PatchDiameter) - int2(PatchDistance, PatchDistance);
		int2 Q = Position + Offset;
#if (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NONE) \
	|| (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NUM_OF_WEIGHTS_PER_PIXELxWxH)
		uint BufferIndex = BufferOffset + i;
#else
		uint BufferIndex = GetWeightBufferIndex(DispatchThreadID.xy, i, RegionSize);
#endif
		RWNonLocalMeanWeights[BufferIndex] = GetNonLocalMeanWeight(Position, Q);
	}
}

RWTexture2D<TAtlasValueType> RWNLMWeightAtlas;
int2 NLMWeightAtlasSize;

uint2 GetAtlasPosition(uint3 DispatchThreadID, uint2 TileRegionSize,uint2 Offset = 0)
{
	uint2 TileId = uint2(DispatchThreadID.z % DispatchTileSize.x, DispatchThreadID.z / DispatchTileSize.x);
	uint2 TileOffset = TileId * TileRegionSize;
	uint2 AtlasPosition = TileOffset + DispatchThreadID.xy + Offset;

	return AtlasPosition;
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, THREAD_GROUP_SIZE)]
void NonLocalMeanGetSqauredDistanceToAtlasCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID >= DispatchRegionSize))
	{
		return;
	}

	const float VarianceScale = 1.0f;

	// Query the current position.
	uint2 Position = DispatchThreadID.xy + SeparableFilteringRegion.xy;
	FImageContext PContext = GetImageContext(Position, Image, Variance, VarianceScale);

	// Calculate the squared distance. When source and target is different, we store the 2 consecutive 
	// distances instead of one pair.
#define DISTANCE_QUERY_COUNT ((NONLOCALMEAN_SEPARATE_SOURCE+1)*NONLOCALMEAN_ATLAS_SYMMETRIC_PAIR_COUNT)

	int BaseOffsetIndex = (DispatchId * DispatchTileCount + DispatchThreadID.z) * DISTANCE_QUERY_COUNT;
	int PatchSearchWidth = PatchDistance * 2 + 1;

	TAtlasValueType Distances = (TAtlasValueType)0;
	FImageContext QContext;

	UNROLL
	for (int i = 0; i < DISTANCE_QUERY_COUNT; ++i)
	{
		int OffsetIndex = BaseOffsetIndex + i;

		int2 Offset = int2(OffsetIndex % PatchSearchWidth, OffsetIndex / PatchSearchWidth) - int2(PatchDistance, PatchDistance);
		int2 Q = Position + Offset;

		QContext = GetImageContext(Q, TargetImage, TargetVariance, VarianceScale);
#if NONLOCALMEAN_SEPARATE_SOURCE
		float Distance = GetSquaredDistance(PContext, QContext);
		Distances[i] = Distance;
#else
		float2 Distance = GetSymmetricSquaredDistance(PContext, QContext);
		Distances[i * 2 + 0] = Distance.x;
		Distances[i * 2 + 1] = Distance.y;
#endif
	}

	uint2 AtlasPosition = GetAtlasPosition(DispatchThreadID, DispatchRegionSize.xy);
	RWNLMWeightAtlas[AtlasPosition] = Distances;
}

RWBuffer<TAtlasValueType> RWNLMWeights;
Texture2D<TAtlasValueType> NLMWeightAtlasSource;
RWTexture2D<TAtlasValueType> RWNLMWeightAtlasTarget;
int3 SeperableRegionSize;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, THREAD_GROUP_SIZE)]
void NonLocalMeanSeperableFilterPatchSqauredDistanceCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID >= SeperableRegionSize))
	{
		return;
	}

#if NONLOCALMEAN_SEPRERABLE_PASS == NONLOCALMEAN_SEPERABLE_FILTER_HORIZONTAL
	int2 Direction = int2(1, 0);
	uint2 Offset = 0;
#elif NONLOCALMEAN_SEPRERABLE_PASS == NONLOCALMEAN_SEPERABLE_FILTER_VERTICAL
	int2 Direction = int2(0, 1);
	uint2 Offset = uint2(PatchSize,PatchSize); // Don't need to run the vertical filter  for the outside patch region.
#endif

	uint2 AtlasPosition = GetAtlasPosition(DispatchThreadID, DispatchRegionSize.xy, Offset);

	TAtlasValueType Sum = (TAtlasValueType)0;
	for (int i = -PatchSize; i <= PatchSize; ++i)
	{
		int2 Pij = AtlasPosition + i * Direction;
		Sum += NLMWeightAtlasSource[Pij];
	}

	Sum /= (2 * PatchSize + 1);

#if NONLOCALMEAN_SEPRERABLE_PASS == NONLOCALMEAN_SEPERABLE_FILTER_HORIZONTAL
	RWNLMWeightAtlasTarget[AtlasPosition] = Sum;
#elif NONLOCALMEAN_SEPRERABLE_PASS == NONLOCALMEAN_SEPERABLE_FILTER_VERTICAL
	uint2 Q = 0;
	TAtlasValueType PatchSquareDistance = max(0, Sum);
	TAtlasValueType PatchWeight0 = GetNonLocalMeanWeight(AtlasPosition, Q, PatchSquareDistance);
	
	// Write to buffer with dimension layout XxYxNumB (each element is of size Count(TAtlasValueType,float)/2)
#if BUFFER_PASS_THROUGH
	const uint x = DispatchThreadID.x - (PatchDistance);
	const uint y = DispatchThreadID.y - (PatchDistance);
	const uint X = SeperableRegionSize.x - 2 * PatchDistance;
	const uint Y = SeperableRegionSize.y - 2 * PatchDistance;
#else
	const uint x = DispatchThreadID.x;
	const uint y = DispatchThreadID.y;
	const uint X = SeperableRegionSize.x;
	const uint Y = SeperableRegionSize.y;
#endif
	const uint z = DispatchId * DispatchTileCount + DispatchThreadID.z;
	const uint SaveIndex = x + X * (y + z * Y);

#if BUFFER_PASS_THROUGH
	const uint MaxZ = (Pow2(2 * PatchDistance + 1) - 1 + 4) / 4;
	if (x>=0 && y >=0 && x < X && y < Y && z < MaxZ)
#endif
	{
		RWNLMWeights[SaveIndex] = PatchWeight0;
	}
#endif
}

Buffer<float2> SourceBuffer;
RWBuffer<float> RWTargetBuffer;
int4 SourceBufferDim;// B,X+2*pd, Y+2*pd, Wb
int3 TargetBufferDim;// W, X, Y
int HalfOffsetSearchCount;

float2 GetInputFromBuffer(uint x, uint y, uint w)
{
	const uint B = SourceBufferDim.x;
	const uint X = SourceBufferDim.y;
	const uint Y = SourceBufferDim.z;
	const uint Wb = SourceBufferDim.w;

	const uint b = w % B;
	const uint wb = w / B;

	const uint InputIndex = b + B * (x + X * (y + Y * wb));
	return SourceBuffer[InputIndex];
}

void WriteWeight(uint2 PixelPosition, uint2 WeightPosition, float Weight)
{
	const uint PatchDiemeter = 2 * PatchDistance + 1;
	uint WeightIndex = WeightPosition.x + WeightPosition.y * PatchDiemeter;

#if (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NONE) \
	|| (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NUM_OF_WEIGHTS_PER_PIXELxWxH)
	uint OutputIndex = WeightIndex + PixelPosition.x * TargetBufferDim.x + PixelPosition.y * TargetBufferDim.x * TargetBufferDim.y;
#elif NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_WxHxNUM_OF_WEIGHTS_PER_PIXEL
	uint OutputIndex = PixelPosition.x + TargetBufferDim.y * (PixelPosition.y + TargetBufferDim.z * WeightIndex);
#elif NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4
	uint StartIndex = WeightIndex / 4;
	uint WeightOffset = WeightIndex % 4;
	uint OutputIndex = WeightOffset + 4 * (PixelPosition.x + TargetBufferDim.y * (PixelPosition.y + TargetBufferDim.z * StartIndex));
#endif
	RWTargetBuffer[OutputIndex] = Weight;
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, THREAD_GROUP_SIZE)]
void NonLocalMeanReshapeBufferCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	uint3 ReshapeDim = uint3(SourceBufferDim.y, SourceBufferDim.z, HalfOffsetSearchCount);
	if (any(DispatchThreadID >= ReshapeDim.xyz))
	{
		return;
	}

	const uint x2pd = DispatchThreadID.x;
	const uint y2pd = DispatchThreadID.y;
	const uint w = DispatchThreadID.z;

	// Get from input
	// For seperate source
	// both .x and .y stores Source(x,y)->Target(P)
	// For same image
	// .x stores (x,y)->P, weights
	// .y stores P ->(x,y) weights
	const float2 PatchWeights = GetInputFromBuffer(x2pd, y2pd, w);
	const uint PatchWidth = 2 * PatchDistance + 1;
	int2 CoordinateXY2pd = int2(x2pd, y2pd);
	int2 CoordinateXY = CoordinateXY2pd - PatchDistance;
	int2 RegionSize = TargetBufferDim.yz;

#if NONLOCALMEAN_SEPARATE_SOURCE 
	for (int i = 0; i < 2; ++i)
	{
		int WeightIndex = (2 * w + i);
		if (WeightIndex < TargetBufferDim.x && all(CoordinateXY < RegionSize) && all(CoordinateXY >= 0))
		{
			int2 WeightCoordinateP = int2(WeightIndex % PatchWidth, WeightIndex / PatchWidth);
			WriteWeight(CoordinateXY, WeightCoordinateP, PatchWeights[i]);
		}
	}
#else
	// For a given w index in upper matrix, convert to (i,j), find the mapping to lower coordinate.
	int2 WeightCoordinateP = int2 (w % PatchWidth, w / PatchWidth);
	const int2 WeightCenterPoint = int2(PatchDistance, PatchDistance);
	int2 DeltaXY = WeightCoordinateP - WeightCenterPoint;
	int2 WeightCoordinateQ = WeightCenterPoint - DeltaXY;
	int2 CoordinateP2pd = CoordinateXY2pd + DeltaXY;

	// Save (x,y) -> P weights
	if (all(CoordinateXY < RegionSize) && all(CoordinateXY >= 0))
	{
		WriteWeight(CoordinateXY, WeightCoordinateP, PatchWeights.x);
	}
	// Save P -> (x,y) weights if it's inside the region.
	int2 CoordinateP = CoordinateP2pd - PatchDistance;
	if (all(CoordinateP < RegionSize) && all (CoordinateP >= 0))
	{
		WriteWeight(CoordinateP, WeightCoordinateQ, PatchWeights.y);
	}
#endif
}

//------------------------------------------------------------------------------------------------------
// Collaborative filtering
//	1. Tiling
//------------------------------------------------------------------------------------------------------

RWBuffer<float> Dest;
int CopyChannelOffset;
int CopyChannelCount;
int BufferChannelOffset;
int BufferChannelSize;
int4 CopyRegion;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void CopyTextureToBufferCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	uint2 CopyRegionSize = uint2(CopyRegion.z - CopyRegion.x, CopyRegion.w - CopyRegion.y);
	if (any(DispatchThreadID.xy >= CopyRegionSize))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy + CopyRegion.xy;
	const int BufferOffset = (DispatchThreadID.x + DispatchThreadID.y * CopyRegionSize.x) * BufferChannelSize + BufferChannelOffset;

	// If copy region is outside of the texture region, get a mirror.
	TPixelValue  PixelValue = Source.Load(uint3(GetMirroredPosition(Position, TextureSize), 0));

	const int EndCopyChannelOffset = CopyChannelOffset + CopyChannelCount;

#if SOURCE_CHANNEL_COUNT == 1
	Dest[BufferOffset] = PixelValue;
#else
	for (int i = CopyChannelOffset; i < EndCopyChannelOffset; ++i)
	{
		Dest[BufferOffset + (i - CopyChannelOffset)] = PixelValue[i];
	}
#endif
}

RWTexture2D<float4> RWSource;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void NormalizeTextureCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}
	
	const uint2 Position = DispatchThreadID.xy;
	float4 Texel = RWSource[Position];
	
	if (Texel.w == 0)
	{
		Texel.w = 1;
	}

	RWSource[Position] = Texel / Texel.w;
}

//------------------------------------------------------------------------------------------------------
//	2. Weighted Least-square solver

Buffer<float> X;
Buffer<TWeightReadType> W;
Buffer<float> Y;

RWBuffer<float> Result;

int2 XDim;
int WDim;
int NumOfTemporalFrames;
int NumOfWeigthsPerPixelPerFrame;

#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_GENERALIZED
int2 YDim;
#else
#define YDim XDim
#endif

// X, Y is stored per pixel, in each pixel, it stores the pixel footage for each temporal frame
// Pixel layout example with two data per temporal frame
// P1                     ...   Pn
// |F0 F1|F0 F1|F0 F1|
//   T0     T1   T2
// 
// W stores one weight per pixel, when a frame stored, the next frame is stored 
// P1 ... Pn | P1 ... Pn|
// T0          T1        ...

float GetX(uint2 Position, uint f)
{
	const uint Width = TextureSize.x + 2 * PatchDistance;
	const uint F = XDim.y;

	const uint Index = (Position.y * Width + Position.x) * (F * NumOfTemporalFrames) + f;
	return X[Index];
}

float GetX(uint2 Position, uint Db, uint FrameIndex)
{
	const uint F = XDim.y;
	return GetX(Position, FrameIndex * F + Db);
}

float GetY(int2 Position, int f)
{
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_GENERALIZED
	const float Width = TextureSize.x + 2 * PatchDistance;
	const float F = YDim.y;

	const int Index = (Position.y * Width + Position.x) * (F * NumOfTemporalFrames) + f;
	return Y[Index];
#else
	return GetX(Position, f);
#endif
}

TWeightReadType GetW(int2 Position, int WeightIndex, int FrameIndex)
{
	const int NumberOfPixels = TextureSize.x * TextureSize.y;

#if (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NONE) \
	|| (NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_NUM_OF_WEIGHTS_PER_PIXELxWxH)
	const int Index = NumOfWeigthsPerPixelPerFrame *(NumberOfPixels * FrameIndex + (Position.y * TextureSize.x + Position.x)) + WeightIndex;

#elif NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_WxHxNUM_OF_WEIGHTS_PER_PIXEL
	const int Index = NumOfWeigthsPerPixelPerFrame * NumberOfPixels * FrameIndex
					  + Position.x + TextureSize.x * (Position.y + TextureSize.y * WeightIndex);

#elif NLM_WEIGHTLAYOUT == NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4
	const uint FrameOffset = NumberOfPixels * (NumOfWeigthsPerPixelPerFrame + 4 - 1) / 4;
	uint StartIndex = WeightIndex / 4; // return the weight stride instead of the specific weight
	uint WeightOffset = WeightIndex % 4;
	const uint Index = FrameOffset * FrameIndex
					   + (Position.x + TextureSize.x * (Position.y + TextureSize.y * StartIndex));
#else
	#error not implemented
#endif
	return W[Index];
}

int2 GetModifiedXyYyDimension()
{
	const int ModifiedXyDim = XDim.y + APPEND_CONSTANT_DIMENSION_TO_X;
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_GENERALIZED
	const int ModifiedYyDim = YDim.y;
#else
	const int ModifiedYyDim = YDim.y + APPEND_CONSTANT_DIMENSION_TO_X;
#endif

	return int2(ModifiedXyDim, ModifiedYyDim);
}

void WriteResult(int2 Position, int f, float Value)
{
	const int2 ModifiedXyYyDim = GetModifiedXyYyDimension();
	const int Index = (Position.y * TextureSize.x + Position.x) * (ModifiedXyYyDim.x * ModifiedXyYyDim.y) + f;
	Result[Index] = Value;
}

#include "/Engine/Private/DoubleFloat.ush"

int SamplingStep;

void SolveMatrixMultiplicationPerMatrixItem(int2 Position, int2 ModifiedXyYyDim, int PatchWidth, int SingleFrameDataCount)
{
#if NLM_WEIGHTLAYOUT != NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4
	for (int i = 0; i < ModifiedXyYyDim.x; ++i)
	{
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC
		for (int k = i; k < ModifiedXyYyDim.y; ++k)
#else
		for (int k = 0; k < ModifiedXyYyDim.y; ++k)
#endif
		{
			FDFScalar R = (FDFScalar)0;
			for (int j = 0; j < XDim.x; ++j)
			{

				int IndexWithinFrame = j % SingleFrameDataCount;
				int2 Offset = int2(IndexWithinFrame % PatchWidth, IndexWithinFrame / PatchWidth);
				int frame_index = j / SingleFrameDataCount;

				float w = GetW(Position, IndexWithinFrame, frame_index); // get w, cache ???, 19*19*T

				int2 TargetPosition = Position + Offset;

#if APPEND_CONSTANT_DIMENSION_TO_X
				float x = 1.0f;
				float y = 1.0f;

				BRANCH
					if (i < XDim.y)
					{
						x = GetX(TargetPosition, XDim.y * frame_index + i);
					}

				BRANCH
					if (k < YDim.y)
					{
						y = GetY(TargetPosition, YDim.y * frame_index + k);
					}
#else
				float x = GetX(TargetPosition, XDim.y * frame_index + i); // get x from the ith row X^T
				float y = GetY(TargetPosition, YDim.y * frame_index + k); // get y from the kth column of Y
#endif
				R = DFAdd(R, DFMultiply(DFTwoSum(x, 0.0f), DFMultiply(DFTwoSum(sqrt(w), 0.0f), DFTwoSum(y, 0.0f))));
			}

			float RResolved = DFDemote(R);
			WriteResult(Position, i * ModifiedXyYyDim.y + k, RResolved);

#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC
			WriteResult(Position, k * ModifiedXyYyDim.y + i, RResolved);
#endif
		}
	}
#endif // NLM_WEIGHTLAYOUT != NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4
}

#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC
#define MATRIX_CACHE_SIZE (((NUM_FEATURE + APPEND_CONSTANT_DIMENSION_TO_X) * (NUM_FEATURE + APPEND_CONSTANT_DIMENSION_TO_X) + 1) / 2)
#elif WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_GENERALIZED
#define MATRIX_CACHE_SIZE ((NUM_FEATURE + APPEND_CONSTANT_DIMENSION_TO_X)*3)
#endif

struct FMatrixCache
{
	float Data[MATRIX_CACHE_SIZE];
};

struct FFeatureXCache
{
	float Data[NUM_FEATURE + APPEND_CONSTANT_DIMENSION_TO_X];
};

struct FFeatureYCache
{
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC
	float Data[NUM_FEATURE + APPEND_CONSTANT_DIMENSION_TO_X];
#else
	float3 Data;
#endif
};

void FillFeatureXCache(out FFeatureXCache FeatureCache,uint2 Position, int2 ModifiedXyYyDim, uint FrameFeatureOffset)
{
	for (int i = 0; i < XDim.y; ++i)
	{
		FeatureCache.Data[i] = GetX(Position, FrameFeatureOffset + i); 
	}

#if APPEND_CONSTANT_DIMENSION_TO_X
	FeatureCache.Data[ModifiedXyYyDim.x - 1] = 1.0f;
#endif
}

void FillFeatureYCache(out FFeatureYCache FeatureCache, uint2 Position, int2 ModifiedXyYyDim, uint FrameFeatureOffset)
{
	for (int i = 0; i < YDim.y; ++i)
	{
		FeatureCache.Data[i] = GetY(Position, FrameFeatureOffset + i);
	}
}

int SourceFrameIndex;

void MatrixMultiplicationSmallMatrixOptimization(int2 Position, int2 ModifiedXyYyDim, int PatchWidth, int SingleFrameDataCount)
{	
	FMatrixCache MatrixCache = (FMatrixCache)0;

	int NumOfFrames = XDim.x / SingleFrameDataCount;
	int Index = 0;

#if USE_SAMPLING_STEP && (NLM_WEIGHTLAYOUT != NLM_WEIGHTLAYOUT_4xWxHxNUM_OF_WEIGHTS_BY_4)
	#if SAMPLING_TYPE == SAMPLING_TYPE_INCREMENTAL
	//This sampling might lead to incorrect DOF when there is ambiguity using feature to predict
	//non-DOF background in previous/future frames, and DOF foreground that is the current frame.
	//we might need to apply regression for each frame instead of using combined frames.
	for (int step = 0; step < XDim.x; step+=SamplingStep)
	{
		int IndexWithinFrame = step / NumOfFrames;
		int2 Offset = int2(IndexWithinFrame % PatchWidth, IndexWithinFrame / PatchWidth);
		int2 TargetPosition = Position + Offset;

		int t = step % NumOfFrames;
	#elif SAMPLING_TYPE == SAMPLING_TYPE_FOCUS_CURRENT_FRAME
		// focus sampling more to the current frames.
	const int SourceFrame = SourceFrameIndex;

	int PreviousSamplingFrame = NumOfFrames - 1;
	int IndexWithinFrame = 0;
	for (int step = 0; step < XDim.x && IndexWithinFrame < SingleFrameDataCount-1; step += SamplingStep)
	{
		int t = step % NumOfFrames;
		if (t < PreviousSamplingFrame && NumOfFrames > 1)
		{
			PreviousSamplingFrame = t;
			t = SourceFrame;
			IndexWithinFrame += 1;
		}
		else
		{
			int Tmp = t;
			t = SourceFrame;
			PreviousSamplingFrame = Tmp;
		}

		int2 Offset = int2(IndexWithinFrame % PatchWidth, IndexWithinFrame / PatchWidth);
		int2 TargetPosition = Position + Offset;
	#endif
#else
	for (int j = 0; j < SingleFrameDataCount; j += WEIGHT_PIXEL_INCREMENT)
	{
		int IndexWithinFrame = j;
		int2 Offset = int2(IndexWithinFrame % PatchWidth, IndexWithinFrame / PatchWidth);
		int2 TargetPosition = Position + Offset;

		for (int t = 0; t < NumOfFrames; ++t)
#endif
		{
			int frame_index = t;
			TWeightReadType sqrtw = sqrt(GetW(Position, IndexWithinFrame, frame_index)); // get w.
			
			UNROLL
			for (int WeightStrideIndex = 0; WeightStrideIndex < WEIGHT_PIXEL_INCREMENT; ++WeightStrideIndex)
			{
#if	WEIGHT_PIXEL_INCREMENT == 1
				float LocalWeight = sqrtw;
#else
				IndexWithinFrame = j + WeightStrideIndex;
				Offset = int2(IndexWithinFrame % PatchWidth, IndexWithinFrame / PatchWidth);
				TargetPosition = Position + Offset;
				float LocalWeight = sqrtw[WeightStrideIndex];

				if (IndexWithinFrame < SingleFrameDataCount)
#endif
				{
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_GENERALIZED
					FFeatureYCache Y;
					FillFeatureYCache(Y, TargetPosition, ModifiedXyYyDim, YDim.y * frame_index);
#endif
					FFeatureXCache X;
					FillFeatureXCache(X, TargetPosition, ModifiedXyYyDim, XDim.y * frame_index);

					Index = 0;
					for (int i = 0; i < ModifiedXyYyDim.x; ++i)
					{
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC
						for (int k = i; k < ModifiedXyYyDim.y; ++k)
#else
						for (int k = 0; k < ModifiedXyYyDim.y; ++k)
#endif
						{
							float x = X.Data[i];

#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_GENERALIZED
							float y = Y.Data[k];
#else 
							float y = X.Data[k];
#endif
							MatrixCache.Data[Index] += x * LocalWeight * y;
							++Index;
						}
					}
				}
			}
		}
	}

	Index = 0;
	for (int i = 0; i < ModifiedXyYyDim.x; ++i)
	{
#if WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC
		for (int k = i; k < ModifiedXyYyDim.y; ++k)
#else
		for (int k = 0; k < ModifiedXyYyDim.y; ++k)
#endif
		{
			float RResolved = MatrixCache.Data[Index++];
			WriteResult(Position, i * ModifiedXyYyDim.y + k, RResolved);

			#if (WEIGHTED_MULTIPLICATION_TYPE == WEIGHTED_MULTIPLICATION_QUADRATIC)
				WriteResult(Position, k * ModifiedXyYyDim.y + i, RResolved);
			#endif
		}
	}
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void InPlaceBatchedMatrixMultiplicationCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}
	int2 Position = DispatchThreadID.xy;
	
	int PatchWidth = 2 * PatchDistance + 1;
	int SingleFrameDataCount = Pow2(2 * PatchDistance + 1);//TODO
	
	//======================================================================================================
	// For each pixel, solve the matrix X^T W * Y or X^T W * X
	// asymptotic complexity O(n^3).
	// Actual: FxAxN = 21*N  for X^T W Y, where N = 19*19*T, where T = 1,3,5
	//         FxNxN = 21*N^2 for X^T W X
	// The time spent for solving the quadratic form increases quadratically, 1, 9, 25.
	// Improvement: 
	// 1. Based on "one in ten/twenty rules", we might not need that many Ns.
	// E.g., if 7x3 weights needed to be predicted, we might shrink 19*19*5 to 19*19*(1~2) observations that is 
	// distributed over 5 frames.
	const int2 ModifiedXyYyDim = GetModifiedXyYyDimension();

#if SMALL_MATRIX_OPTIMIZE
	MatrixMultiplicationSmallMatrixOptimization(Position, ModifiedXyYyDim, PatchWidth, SingleFrameDataCount);
#else
	SolveMatrixMultiplicationPerMatrixItem(Position, ModifiedXyYyDim, PatchWidth, SingleFrameDataCount);
#endif
}

#if LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_NEWTON_SCHULZ

#define NUM_NEWTON_SCHULTZ_ITERATIONS 20

#define NEWTON_INITIAL_GUESS_TYPE INITIAL_GUESS_EUCLIDEAN_NORM

#elif LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_NEWTON_CHOLESKY

// lambda = 2.5e-3, newton iteration count = 3, equivilent to 20 newton iterations.
#define NUM_NEWTON_SCHULTZ_ITERATIONS 3

#define NEWTON_INITIAL_GUESS_TYPE INITIAL_GUESS_INVERSE_CHOLESKY_DECOMPOSITION

#endif

#define MATRIX_DIM NUM_FEATURE
#include "NFORRegression.ush"

float Lambda;
RWBuffer<uint> RWSuccessAndFailIndexBuffer;

void LinearSolveEntry(uint AOffset, uint ASize, uint BOffset, uint MatrixIndex)
{

#if (LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_NEWTON_SCHULZ || \
	LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_NEWTON_CHOLESKY)
	bool bComplete = NewtonIterativeSolve(AOffset, ASize, BOffset, Lambda, Result);
#elif LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_CHOLESKY
	bool bComplete = CholeskeySolve(AOffset, ASize, BOffset, Lambda, Result);
#endif

#if OUTPUT_INDICES
	bool WriteCountIndexLocation = bComplete ? OUTPUT_MATRIX_TYPE_SUCCESS : OUTPUT_MATRIX_TYPE_FAIL;

	uint IndexToStore = 0;
	InterlockedAdd(RWSuccessAndFailIndexBuffer[WriteCountIndexLocation], 1, IndexToStore);
	IndexToStore = lerp(IndexToStore, (NumOfElements - IndexToStore - 1), WriteCountIndexLocation);
	RWSuccessAndFailIndexBuffer[IndexToStore + 2] = MatrixIndex;
#endif

}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void LinearSolverCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const int MatrixIndex = DispatchThreadID.x + DispatchThreadID.y * NumOfElementsPerRow;
	if (MatrixIndex >= NumOfElements || DispatchThreadID.x >= NumOfElementsPerRow)
	{
		return;
	}
	const int BSize = BDim.x * BDim.y;
	const int BOffset = MatrixIndex * BSize;
	const int ASize = ADim.x * ADim.y;
	const int AOffset = MatrixIndex * ASize;

	LinearSolveEntry(AOffset, ASize, BOffset, MatrixIndex);
}


RWBuffer<uint> RWIndirectDispatchArgsBuffer;
Buffer<uint> SuccessAndFailIndexBuffer;

[numthreads(1, 1, 1)]
void LinearSolverBuildIndirectDispatchArgsCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint NumOfIndices = SuccessAndFailIndexBuffer[INPUT_MATRIX_TYPE];

	if (NumOfIndices > 0)
	{
		const uint NumOfGroupsX = (NumOfIndices + (THREAD_GROUP_SIZE * THREAD_GROUP_SIZE) - 1) / (THREAD_GROUP_SIZE * THREAD_GROUP_SIZE);
		WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, 0, NumOfGroupsX, 1, 1);
	}
	else
	{
		WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, 0, 0, 0, 0);
	}
}

groupshared uint CompactNumOfElements;

[numthreads(THREAD_GROUP_SIZE*THREAD_GROUP_SIZE, 1, 1)]
void LinearSolverIndirectCS(in const uint3 DispatchThreadID : SV_DispatchThreadID, uint GI : SV_GroupIndex)
{
	if (GI == 0)
	{
		CompactNumOfElements = SuccessAndFailIndexBuffer[INPUT_MATRIX_TYPE];
	}

	GroupMemoryBarrierWithGroupSync();

	const uint ElementIndex = DispatchThreadID.x;
	if (ElementIndex >= CompactNumOfElements)
	{
		return;
	}

	const uint ResolvedElementIndex = lerp(ElementIndex, NumOfElements - 1 - ElementIndex, INPUT_MATRIX_TYPE);
	const uint MatrixIndex = SuccessAndFailIndexBuffer[2 + ResolvedElementIndex];

	const int BSize = BDim.x * BDim.y;
	const int BOffset = MatrixIndex * BSize;
	const int ASize = ADim.x * ADim.y;
	const int AOffset = MatrixIndex * ASize;

	LinearSolveEntry(AOffset, ASize, BOffset, MatrixIndex);
}

//===============================================================================================
// Reconstruct spatial temporal images
int FrameIndex;
float AlbedoOffset;
RWTexture2D<float4> RWReconstruction;
RWStructuredBuffer<uint4> RWReconstructBuffer;
RWTexture2D<UlongType> RWReconstructBuffer64;

float GetB(int2 Position, int f)
{
	const int Index = (Position.y * TextureSize.x + Position.x) * (BDim.x*BDim.y) + f;
	return B[Index];
}

uint4 EncodeFloat4(float4 Value)
{
	uint4 fractions = frac(max(Value,0.0f))*0xffff;
	uint4 Integers = uint4(max(Value, 0.0f));
	return Integers<<16 | (fractions & 0xffff);
}

float4 DecodeFloat4(uint4 Value)
{
	float4 Decoded = float4(Value>>16) + float4(Value & 0xffff)/65536;
	return Decoded;
}

void SaveFloat4ToBufferOrTexture(uint2 Position, float4 Value, uint NumOfScattering)
{
	Value = max(Value, 0);
#if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
	float4 XBWFraction = frac(Value);
	uint4 XBWInteger = Value - XBWFraction;
	XBWInteger = min(XBWInteger, 0xffffffff / NumOfScattering); // Avoid overflow.

	uint4 XBWFractionAsInteger = uint4(XBWFraction * 0xffffffff);
	{
		const UlongType R = PackUlongType(uint2(XBWFractionAsInteger.r, XBWInteger.r));
		ImageInterlockedAddUInt64(RWReconstructBuffer64, uint2(Position.x * 4 + 0, Position.y), R);

		const UlongType G = PackUlongType(uint2(XBWFractionAsInteger.g, XBWInteger.g));
		ImageInterlockedAddUInt64(RWReconstructBuffer64, uint2(Position.x * 4 + 1, Position.y), G);

		const UlongType B = PackUlongType(uint2(XBWFractionAsInteger.b, XBWInteger.b));
		ImageInterlockedAddUInt64(RWReconstructBuffer64, uint2(Position.x * 4 + 2, Position.y), B);

		const UlongType A = PackUlongType(uint2(XBWFractionAsInteger.a, XBWInteger.a));
		ImageInterlockedAddUInt64(RWReconstructBuffer64, uint2(Position.x * 4 + 3, Position.y), A);
	}

#else
	int IntermediateBufferIndex = Position.x + Position.y * (TextureSize.x + 2 * PatchDistance);
	uint4 XBWEncoded = EncodeFloat4(Value);
	float4 OldValue = 0;
	InterlockedAdd(RWReconstructBuffer[IntermediateBufferIndex].x, XBWEncoded.x, OldValue.x);
	InterlockedAdd(RWReconstructBuffer[IntermediateBufferIndex].y, XBWEncoded.y, OldValue.y);
	InterlockedAdd(RWReconstructBuffer[IntermediateBufferIndex].z, XBWEncoded.z, OldValue.z);
	InterlockedAdd(RWReconstructBuffer[IntermediateBufferIndex].w, XBWEncoded.w, OldValue.w);

#endif //COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void ReconstructSpatialTemporalImageCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}
	int2 Position = DispatchThreadID.xy;

	const int NumOfFeatures = XDim.y;
	const int PatchSideLength = 2 * PatchDistance + 1;

	// Get the weights for all the pixel's patch's reconstruction
	const int BWeightStride = BDim.x * BDim.y;

	float3 Weights[NUM_FEATURE]; // 6x3 matrix.
	
	// checkf(NUM_FEATURE == XDim.y)

	UNROLL
	for (int i = 0; i < NUM_FEATURE; ++i)
	{
		float3 weight = 0;

		weight.x = GetB(Position, i * BDim.y + 0);
		weight.y = GetB(Position, i * BDim.y + 1);
		weight.z = GetB(Position, i * BDim.y + 2);
		Weights[i] = weight;
	}

	const int WeightStride = Pow2(PatchSideLength);
	const int XStride = XDim.y * WeightStride;
	float4 GatheredValue = 0;

#if PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_EACH
	float3 Albedo = 0;
#endif

	// For center frame, reconstruct and scatter the result
	for (int PixelIndex = 0; PixelIndex < WeightStride; PixelIndex += WEIGHT_PIXEL_INCREMENT)
	{
		int x = PixelIndex % PatchSideLength;
		int y = PixelIndex / PatchSideLength;
		TWeightReadType PixelWeights = GetW(Position, PixelIndex, FrameIndex);
		
		UNROLL
		for (int SubStrideIndex = 0; SubStrideIndex < WEIGHT_PIXEL_INCREMENT; ++SubStrideIndex)
		{
#if WEIGHT_PIXEL_INCREMENT == 1
			float LocalWeight = PixelWeights;
#else
			int ResolvedPixelIndex = PixelIndex + SubStrideIndex;
			x = ResolvedPixelIndex % PatchSideLength;
			y = ResolvedPixelIndex / PatchSideLength;
			float LocalWeight = PixelWeights[SubStrideIndex];
			if (ResolvedPixelIndex < WeightStride)
#endif
			{
				uint2 TargetPosition = Position + int2(x, y);
				float3 Denoised = 0;

				for (int i = 0; i < NUM_FEATURE; ++i)
				{
					float FeatureI = 1.0f;
					if (i < NumOfFeatures)
					{
						FeatureI = GetX(TargetPosition, i, FrameIndex);

#if PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_EACH
						//Assume the first three channels are albedo. TODO: use passed in index
						if (i < 3)
						{
							Albedo[i] = FeatureI;
						}
#endif
					}
					Denoised += FeatureI * Weights[i];
				}

#if RECONSTRUCTION_TYPE == RECONSTRUCTION_TYPE_SCATTER

#if PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_EACH
				Denoised *= RADIANCE_PREPROCESS_SCALE_FACTOR * Albedo;
#elif PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_FINAL
				Denoised *= RADIANCE_PREPROCESS_SCALE_FACTOR;
#endif
				// Scatter the result instead of gathering for the current frame to get high frequency results (better reconstruction).
				float4 XBW = float4(Denoised * LocalWeight, LocalWeight);
				SaveFloat4ToBufferOrTexture(TargetPosition, XBW, WeightStride);

#elif RECONSTRUCTION_TYPE == RECONSTRUCTION_TYPE_GATHER

#if PRE_ALBEDO_DIVIDE == PRE_ALBEDO_DIVIDE_EACH
				Denoised *= Albedo;
#endif
				GatheredValue += float4(Denoised * LocalWeight, LocalWeight);
#endif
			}
		}
	}
#if RECONSTRUCTION_TYPE == RECONSTRUCTION_TYPE_GATHER
	// For other frames, after reconstruction, perform reconstruction with the weights.
	RWReconstruction[Position + PatchDistance] += GatheredValue;
#endif

}

StructuredBuffer<uint4> StructuredBufferSource;
Texture2D<UlongType> ReconstructBuffer64;

float DecodeUint64ToFloat(Texture2D<UlongType> inTexture, uint2 Position)
{
	const uint2 REncoded = UnpackUlongType(inTexture[Position]);
	return REncoded.y + float(REncoded.x) / 0xffffffff;
}

float4 DecodeFloat4FromBufferOrTexture(uint2 Position)
{
	float4 DecodedValue = 0;
#if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
	float R = DecodeUint64ToFloat(ReconstructBuffer64, uint2(Position.x * 4 + 0, Position.y));
	float G = DecodeUint64ToFloat(ReconstructBuffer64, uint2(Position.x * 4 + 1, Position.y));
	float B = DecodeUint64ToFloat(ReconstructBuffer64, uint2(Position.x * 4 + 2, Position.y));
	float A = DecodeUint64ToFloat(ReconstructBuffer64, uint2(Position.x * 4 + 3, Position.y));

	DecodedValue = float4(R,G,B,A);

#if PRE_ALBEDO_DIVIDE != PRE_ALBEDO_DIVIDE_DISABLED
	DecodedValue.rgb *= RADIANCE_POSTPROCESS_INVERSE_SCALE_FACTOR;
#endif

#else
	int BufferIndex = Position.x + Position.y * TextureSize.x;
	DecodedValue = DecodeFloat4(StructuredBufferSource[BufferIndex]);
#endif

	// There would be cases where a is 0 due to w being too small and encoded as 0.
	// Without rejecting this sample, could see borders artifacts around each tile.
	DecodedValue = lerp(0, DecodedValue, DecodedValue.a > 0);

	return DecodedValue;
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void AccumulateBufferToTextureCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}

	uint2 Position = DispatchThreadID.xy;
	float4 Color = DecodeFloat4FromBufferOrTexture(Position);
	RWTarget[DispatchThreadID.xy] += Color;
}

//------------------------------------------------------------------------------------------------------
//	Bandwidth selection
//------------------------------------------------------------------------------------------------------

Texture2D<TPixelValue> FilteredImage;
RWTexture2D<float> MSE;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void MSEEstimationCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;
	float VarC = GetImageVariance(Position, Variance, VarianceChannelOffset);
	TPixelValue Color = GetImageValue(Position, Image);
	TPixelValue FilteredColor = GetImageValue(Position, FilteredImage);

	// E[(F-C)^2-Var_C] = Bias^2_F + Var_F - 2Cov(C,F)
	// Assume there is low correlation between F and C, and low bias of F
	MSE[Position] = Length2(FilteredColor - Color) - VarC;
}

Texture2D<float> FilteredMSEs_0;
Texture2D<float> FilteredMSEs_1;
RWTexture2D<float> RWSelectionMap;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void GenerateSelectionMapCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;
	float MSE0 = FilteredMSEs_0.Load(int3(Position, 0));
	float MSE1 = FilteredMSEs_1.Load(int3(Position, 0));
	RWSelectionMap[Position] = MSE0 < MSE1 ? 0.0f : 1.0f;
}

Texture2D<float4> FilteredImages_0;
Texture2D<float4> FilteredImages_1;
Texture2D<float>  SelectionMap;
RWTexture2D<float4> RWFilteredImage;

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void CombineFilteredImageCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (any(DispatchThreadID.xy >= TextureSize))
	{
		return;
	}

	const uint2 Position = DispatchThreadID.xy;
	float4 Color0 = FilteredImages_0.Load(int3(Position, 0));
	float4 Color1 = FilteredImages_1.Load(int3(Position, 0));
	float  W = SelectionMap.Load(int3(Position, 0));
	RWFilteredImage[Position] = lerp(Color0, Color1, W);
}
