// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NFORRegression.ush
=============================================================================*/

// TODO: Improve matrix multiplication with Strassem algorithm.
// TODO: Improve the robustness of QR decomposition.
// TODO: Find best iteration number for Newton Schulz Iteration Method.
// TODO: Unroll with constant dimensions.
// TODO: Merge duplicate shader code.

#pragma once

#include "NFORRegressionCommon.ush"

#ifndef MATRIX_DIM
#define MATRIX_DIM 8
#endif

#ifndef NUM_NEWTON_SCHULTZ_ITERATIONS
#define NUM_NEWTON_SCHULTZ_ITERATIONS 3
#endif

#define FULL_MATRIX_SIZE ((MATRIX_DIM)*(MATRIX_DIM))
#define SYMMETRIC_MATRIX_SIZE (((MATRIX_DIM+1)*MATRIX_DIM)/2)
#define LOWER_TRIANGLE_MATRIX_SIZE SYMMETRIC_MATRIX_SIZE

#define QR_DECOMPOSITION_TYPE_DEFAULT		0
#define QR_DECOMPOSITION_TYPE_COLUMN_PIVOT	1

#define QR_DECOMPOSITION_TYPE QR_DECOMPOSITION_TYPE_DEFAULT

Buffer<float> A;
Buffer<float> B;

int2 ADim;
int2 BDim;
int NumOfElements;
int NumOfElementsPerRow;

float MinLambda;

struct FLMatrix
{
	FDFScalar M[LOWER_TRIANGLE_MATRIX_SIZE];
};

#define LMatrixIndex2D(a,b) select(a>b, int2(b,a), int2(a,b))
#define LMatrixIndex1D(i) LMatrixIndex2D((i)%MATRIX_DIM, (i)/MATRIX_DIM)
#define LMatrix(_L,y,x) (_L.M[(x) + (((y) +1) * (y)/2)])

// L^T(x,y) = L(y,x), upper matrix has x>y
#define TLMatrixIndex2D(a,b) select(a>b, int2(a,b), int2(b,a))
#define TLMatrixIndex1D(i) TLMatrixIndex2D((i)%MATRIX_DIM, (i)/MATRIX_DIM)
#define TLMatrix(_L,y,x) LMatrix(_L,x,y)

struct FNFORVector
{
	FDFScalar V[MATRIX_DIM];
};

struct FMatrix
{
	FDFScalar M[FULL_MATRIX_SIZE];
};

#define MATRIX(_A,y,x) (_A.M[y * MATRIX_DIM + x]) 

void LMatrixNegate(inout FLMatrix A)
{
	for (int y = 0; y < MATRIX_DIM; ++y)
	{
		for (int x = 0; x <= y; ++x)
		{
			LMatrix(A, y, x) = DFNegate(LMatrix(A, y, x));
		}
	}
}

// Assume A and B is symmetric and stored in FLMatrix.
// A@B != B@A even if A=A^T, B^T=B
FMatrix  LMatrixMultiply(in FLMatrix A, in FLMatrix B)
{
	FMatrix Ret = (FMatrix)0;

	for (int y = 0; y < MATRIX_DIM; ++y)
	{
		for (int x = 0; x < MATRIX_DIM; ++x)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int k = 0; k < MATRIX_DIM; ++k)
			{
				int2 yk = LMatrixIndex2D(y,k);
				int2 kx = LMatrixIndex2D(k,x);
				Sum = DFAdd(Sum, DFMultiply(LMatrix(A, yk.y, yk.x), LMatrix(B, kx.y, kx.x)));
			}

			MATRIX(Ret, y, x) = Sum;
		}
	}

	return Ret;
}

float LMatrixMaxAbsValue(in FLMatrix A)
{
	float MaxAbsValue = 0;
	for (int i = 0; i < SYMMETRIC_MATRIX_SIZE; ++i)
	{
		MaxAbsValue = max(MaxAbsValue, abs(DFDemote(A.M[i])));
	}
	return MaxAbsValue;
}

// Assume A is symmetric, and B is not. However, A*B is symmetric
FLMatrix  ToLMatrixMultiply(in FLMatrix A, in FMatrix B)
{
	FLMatrix Ret = (FLMatrix)0;

	for (int y = 0; y < MATRIX_DIM; ++y)
	{
		for (int x = 0; x <= y; ++x)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int k = 0; k < MATRIX_DIM; ++k)
			{
				int2 yk = LMatrixIndex2D(y,k);
				Sum = DFAdd(Sum, DFMultiply(LMatrix(A, yk.y, yk.x), MATRIX(B, k, x)));
			}

			LMatrix(Ret, y, x) = Sum;
		}
	}

	return Ret;
}

void Identity(out FMatrix M)
{
	M = (FMatrix)0;

	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		MATRIX(M, i, i) = DFPromote(1.0f);
	}
}

void LMatrixEuclideanNorm(in FLMatrix A, out float EuclideanNorm)
{
	EuclideanNorm = 0.0f;

	for (int i = 0; i < FULL_MATRIX_SIZE; ++i)
	{
		int2 Index2D = LMatrixIndex1D(i);
		EuclideanNorm += Pow2(DFDemote(LMatrix(A, Index2D.y, Index2D.x)));
	}
	EuclideanNorm = sqrt(EuclideanNorm);
}

// |AX-I|. Euclidean norm of AX-I
void MatrixConvergence(in FMatrix A, out float ConvergenceCriteria)
{
	ConvergenceCriteria = 0.0f;

	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int j = 0; j < MATRIX_DIM; ++j)
		{
			ConvergenceCriteria += Pow2(DFDemote(MATRIX(A,i,j))-select(i==j, 1.0f, 0.0f));
		}
	}
	ConvergenceCriteria = sqrt(ConvergenceCriteria);
}


void MatrixNegate(inout FMatrix Matrix)
{
	for (int i = 0; i < FULL_MATRIX_SIZE; ++i)
	{
		Matrix.M[i] = DFNegate(Matrix.M[i]);
	}
}

FMatrix Copy(in FMatrix Matrix)
{
	FMatrix Out;

	for (int i = 0; i < FULL_MATRIX_SIZE; ++i)
	{
		Out.M[i] = Matrix.M[i];
	}

	return Out;
}

FMatrix MatrixMultiply(in FMatrix A, in FMatrix B)
{
	FMatrix Ret;
	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int k = 0; k < MATRIX_DIM; ++k)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int j = 0; j < MATRIX_DIM; ++j)
			{
				const FDFScalar DFMultiplyScalar = DFMultiply(A.M[i * MATRIX_DIM + j], B.M[j * MATRIX_DIM + k]);
				Sum = DFAdd(Sum, DFMultiplyScalar);
			}
			Ret.M[i * MATRIX_DIM + k] = Sum;
		}
	}

	return Ret;
}

FDFScalar GetNFORVectorNorm2(in FDFScalar V[MATRIX_DIM], int N)
{
	FDFScalar L2Norm = (FDFScalar)0;

	for (int i = 0; i < N; ++i)
	{
		L2Norm = DFAdd(L2Norm, DFMultiply(V[i], V[i]));
	}

	L2Norm = DFSqrt(L2Norm);

	return L2Norm;
}

void SwapIntVector(inout int V[MATRIX_DIM], int i, int j)
{
	int Tmp = V[i];
	V[i] = V[j];
	V[j] = Tmp;
}

void SwampColumn(inout FMatrix Matrix, int i, int j)
{
	FDFScalar Tmp;
	for (int row = 0; row < MATRIX_DIM; ++row)
	{
		Tmp = Matrix.M[row * MATRIX_DIM + i];
		Matrix.M[row * MATRIX_DIM + i] = Matrix.M[row * MATRIX_DIM + j];
		Matrix.M[row * MATRIX_DIM + j] = Tmp;
	}
}

FMatrix SelectSubMatrix(in FMatrix Matrix, int StartRowColumnIndex)
{
	FMatrix Ret;
	for (int j = StartRowColumnIndex; j < MATRIX_DIM; ++j)
	{
		for (int k = StartRowColumnIndex; k < MATRIX_DIM; ++k)
		{
			Ret.M[(j - StartRowColumnIndex) * MATRIX_DIM + (k - StartRowColumnIndex)] = Matrix.M[j * MATRIX_DIM + k];
		}
	}
	return Ret;
}

int GetPivot(in FMatrix Matrix, int N)
{
	FNFORVector Vector;

	FDFScalar MaxNorm = DFPromote(-1.0f);
	int MaxIndex = 0;

	for (int column = 0; column < N; ++column)
	{
		for (int row = 0; row < N; ++row)
		{
			Vector.V[row] = Matrix.M[row * MATRIX_DIM + column];
		}
		FDFScalar L2Norm = GetNFORVectorNorm2(Vector.V, N);
		if (DFGreater(L2Norm, MaxNorm))
		{
			MaxNorm = L2Norm;
			MaxIndex = column;
		}
	}

	return MaxIndex + (MATRIX_DIM - N);
}

void HouseHoldReflection(in FDFScalar A[MATRIX_DIM], out FMatrix O, int N)
{
	FDFScalar V[MATRIX_DIM];

	for (int i = 0; i < N; ++i)
	{
		V[i] = A[i];
	}

	FDFScalar L2Norm = GetNFORVectorNorm2(V, N);

	V[0] = DFAdd(V[0], DFMultiply(DFSign(A[0]), L2Norm));

	L2Norm = GetNFORVectorNorm2(V, N);

	FDFScalar InvL2Norm = DFDivide(1.0f, L2Norm);
	for (int i = 0; i < N; ++i)
	{
		V[i] = DFMultiply(V[i], InvL2Norm);
	}

	//I - 2 * vv
	for (int i = 0; i < N; ++i)
	{
		for (int j = 0; j < N; ++j)
		{
			O.M[i * MATRIX_DIM + j] = DFSubtract(lerp(0, 1, i == j), DFMultiply(2, DFMultiply(V[i], V[j])));
		}
	}
}

// Calculate AX=B with (Household) QR decomposition and back substitution.
// TODO: Reduce the memory used in calculation.
void QRDecomposition(in FMatrix A, out FMatrix Q, out FMatrix R
#if QR_DECOMPOSITION_TYPE == QR_DECOMPOSITION_TYPE_COLUMN_PIVOT
	, out int PivotIndices[MATRIX_DIM]
#endif
)
{
	Identity(Q);
	R = Copy(A);

	FMatrix H;
	FMatrix Reflection;
	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		// Select column with the largest norm as pivot.
#if QR_DECOMPOSITION_TYPE == QR_DECOMPOSITION_TYPE_COLUMN_PIVOT
		H = SelectSubMatrix(R, i);
		int Pivot = GetPivot(H, MATRIX_DIM - i);
		SwapIntVector(PivotIndices, i, Pivot);
		SwampColumn(R, i, Pivot);
#endif
		// Apply Householder reflection.
		Identity(H);

		FDFScalar Selection[MATRIX_DIM];
		for (int j = i; j < MATRIX_DIM; ++j)
		{
			Selection[j - i] = R.M[j * MATRIX_DIM + i];
		}

		HouseHoldReflection(Selection, Reflection, MATRIX_DIM - i);

		for (int j = i; j < MATRIX_DIM; ++j)
		{
			for (int k = i; k < MATRIX_DIM; ++k)
			{
				H.M[j * MATRIX_DIM + k] = Reflection.M[(j - i) * MATRIX_DIM + (k - i)];
			}
		}

		Q = MatrixMultiply(Q, H);
		R = MatrixMultiply(H, R);
	}
}

void QR_Solve(int AOffset, int ASize, int BOffset)
{
	FMatrix NA;
	FMatrix NQ;
	FMatrix NR;

#if QR_DECOMPOSITION_TYPE == QR_DECOMPOSITION_TYPE_COLUMN_PIVOT
	int PivotIndices[MATRIX_DIM];

	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		PivotIndices[i] = i;
	}
#endif

	// QR decomposition.
	for (int i = 0; i < FULL_MATRIX_SIZE; ++i)
	{
		NA.M[i] = DFPromote(A[AOffset + i]);
	}

	QRDecomposition(NA, NQ, NR
#if QR_DECOMPOSITION_TYPE == QR_DECOMPOSITION_TYPE_COLUMN_PIVOT
		, PivotIndices
#endif
	);

	// Q^T * B
	FDFScalar Y[MATRIX_DIM * 3];
	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int k = 0; k < 3; ++k)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int j = 0; j < MATRIX_DIM; ++j)
			{
				Sum = DFAdd(Sum, DFMultiply(NQ.M[j * MATRIX_DIM + i], B[BOffset + j * 3 + k]));
			}

			Y[i * 3 + k] = Sum;
		}
	}

	//back substitution for each color.
	// RX = Y, Y = Q^T*B
	FDFScalar X[MATRIX_DIM * 3];

	for (int i = 0; i < (MATRIX_DIM * 3); ++i)
	{
		X[i] = DFPromote(0.0f);
	}

	for (int c = 0; c < 3; ++c)
	{
		for (int i = MATRIX_DIM - 1; i >= 0; --i)
		{
			FDFScalar Ri = NR.M[i * MATRIX_DIM + i];
			if (DFEquals(Ri, 0.0f))
			{
				break;
			}

			FDFScalar Xc = Y[i * 3 + c];

			for (int j = i + 1; j < MATRIX_DIM; ++j)
			{
				Xc = DFSubtract(Xc, DFMultiply(NR.M[i * MATRIX_DIM + j], X[j * 3 + c]));
			}

			X[i * 3 + c] = DFDivide(Xc, Ri);
		}
	}

	//Copy to B matrix.
#if QR_DECOMPOSITION_TYPE == QR_DECOMPOSITION_TYPE_COLUMN_PIVOT
	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int j = 0; j < 3; ++j)
		{
			Result[BOffset + PivotIndices[i] * 3 + j] = DFDemote(X[i * 3 + j]);
		}
	}
#else
	for (int i = 0; i < (MATRIX_DIM * 3); ++i)
	{
		Result[BOffset + i] = DFDemote(X[i]);
	}
#endif

}

bool CholeskeyDecomposition(inout FLMatrix L, FLMatrix AMatrix, float lambda)
{
	bool bComplete = true;
	L = (FLMatrix)0;

	// We need adaptive lambda otherwise, we cannot find a one size fit for all to make 
	// this decomposition always successful.
	// To achieve this, we use the max value of the absolute of the matrtix by the lambda
	lambda *= LMatrixMaxAbsValue(AMatrix);
	
	// In case lambda becomes too small, we use the min lambda to provide sufficient regularization. 
	lambda = max(lambda, MinLambda);
	
	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int j = 0; j <= i; ++j)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int k = 0; k < j; ++k)
			{
				Sum = DFAdd(Sum, DFMultiply(LMatrix(L, i, k), LMatrix(L, j, k)));
			}

			if (i == j)
			{
				// Check Delta > 0
				FDFScalar Delta = DFAdd(LMatrix(AMatrix,i,i), DFSubtract(lambda, Sum));
				bComplete &= DFGreater(Delta, 0) ? true : false;
				LMatrix(L, i, j) = DFSqrt(Delta);
			}
			else
			{
				// Check Ljj != 0
				FDFScalar Ljj = LMatrix(L, j, j);
				bComplete &= DFEquals(Ljj, 0) ? false : true;
				LMatrix(L, i, j) = DFDivide(DFSubtract(LMatrix(AMatrix,i,j), Sum), Ljj);
			}
		}
	}
	return bComplete;
}

FLMatrix CholeskyFactorInverse(FLMatrix L)
{
	FMatrix Y;
	//2. Forward substitution
	// LL^TX=B, LY = B, where B is identity matrix. 
	for (int c = 0; c < MATRIX_DIM; ++c)
	{
		for (int i = 0; i < MATRIX_DIM; ++i)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int j = 0; j < i; ++j)
			{
				Sum = DFAdd(Sum, DFMultiply(MATRIX(Y,j,c), LMatrix(L, i, j)));
			}

			MATRIX(Y,i,c) = DFDivide(DFSubtract(lerp(0.0f, 1.0f, i==c), Sum), LMatrix(L, i, i));
		}
	}

	FLMatrix X = (FLMatrix)0;
	//3. Backward substitution.
	//L^TX = Y
	for (int c = 0; c < MATRIX_DIM; ++c)
	{
		for (int i = MATRIX_DIM - 1; i >= c; --i)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int j = i + 1; j < MATRIX_DIM; ++j)
			{
				int2 jc = TLMatrixIndex2D(j,c);
				Sum = DFAdd(Sum, DFMultiply(TLMatrix(X, jc.y, jc.x), TLMatrix(L, i, j)));
			}

			TLMatrix(X,c,i) = DFDivide(DFSubtract(MATRIX(Y,i,c), Sum), TLMatrix(L, i, i));
		}
	}

	return X;
}

bool CholeskeySolve(int AOffset, int ASize, int BOffset, float lambda, RWBuffer<float> Result)
{
	// Decomposte the matrix to L. As the matrix A might not be positive definite,
	// We solve for (A + \lambda*I)X=B instead, where \lambda is a very small value.

	FLMatrix L = (FLMatrix)0;
	bool bComplete = true;

	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int j = 0; j <= i; ++j)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int k = 0; k < j; ++k)
			{
				Sum = DFAdd(Sum, DFMultiply(LMatrix(L, i, k),LMatrix(L, j, k)));
			}

			if (i == j)
			{
				// Check Delta > 0
				FDFScalar Delta = DFAdd(A[AOffset + i * MATRIX_DIM + i], DFSubtract(lambda, Sum));
				bComplete &= DFGreater(Delta, 0) ? true : false;
				LMatrix(L, i, j) = DFSqrt(Delta);
			}
			else
			{
				// Check Ljj != 0
				FDFScalar Ljj = LMatrix(L, j, j);
				bComplete &= DFEquals(Ljj, 0) ? false : true;
				LMatrix(L, i, j) = DFDivide(DFSubtract(A[AOffset + i * MATRIX_DIM + j], Sum), Ljj);
			}
		}
	}

	FDFScalar Y[MATRIX_DIM * 3];
	//2. Forward substitution
	// LL^TX=B, LY = B, 
	for (int c = 0; c < 3; ++c)
	{
		for (int i = 0; i < MATRIX_DIM; ++i)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int j = 0; j < i; ++j)
			{
				Sum = DFAdd(Sum, DFMultiply(Y[j * 3 + c], LMatrix(L, i, j)));
			}

			Y[i * 3 + c] = DFDivide(DFSubtract(B[BOffset + i * 3 + c], Sum), LMatrix(L, i, i));
		}
	}

	FDFScalar X[MATRIX_DIM * 3];
	//3. Backward substitution.
	//L^TX = Y
	for (int c = 0; c < 3; ++c)
	{
		for (int i = MATRIX_DIM-1; i >= 0; --i)
		{
			FDFScalar Sum = (FDFScalar)0;
			for (int j = i + 1; j < MATRIX_DIM; ++j)
			{
				Sum = DFAdd(Sum, DFMultiply(X[j * 3 + c], TLMatrix(L, i, j)));
			}

			X[i * 3 + c] = DFDivide(DFSubtract(Y[i * 3 + c], Sum), TLMatrix(L, i, i));
		}
	}

	for (int i = 0; i < (MATRIX_DIM * 3); ++i)
	{
		if (isnan(DFDemote(X[i])))
		{
			bComplete = false;
		}
	}

	if (bComplete)
	{
		for (int i = 0; i < (MATRIX_DIM * 3); ++i)
		{
			Result[BOffset + i] = DFDemote(X[i]);
		}
	}

	return bComplete;
}

bool InitializeGuess(inout FLMatrix L, FLMatrix AMatrix, float inLambda)
{
	bool bComplete = true;
#if NEWTON_INITIAL_GUESS_TYPE == INITIAL_GUESS_INVERSE_CHOLESKY_DECOMPOSITION
	bComplete = CholeskeyDecomposition(L, AMatrix, inLambda);

	if (bComplete)
	{
		L = CholeskyFactorInverse(L);
	}

#elif NEWTON_INITIAL_GUESS_TYPE == INITIAL_GUESS_EUCLIDEAN_NORM
	float EuclideanNorm = 0;
	LMatrixEuclideanNorm(AMatrix, EuclideanNorm);

	// Improve the initial guess with identify matrix scaled by Euclidean norm.
	L = (FLMatrix)0;

	for (int x = 0; x < MATRIX_DIM; ++x)
	{
		LMatrix(L, x, x) = DFPromote(1 / EuclideanNorm);
	}
#else
#error not implemented
#endif

	return bComplete;
}

FLMatrix MatrixInverse(FLMatrix AMatrix, float inLambda, inout bool bComplete)
{
	FLMatrix X;

	// Note that X is invertable in the initialization.
	bComplete = InitializeGuess(X, AMatrix, inLambda);


#define NUM_INVERSE_ITERATIONS_DOUBLE NUM_NEWTON_SCHULTZ_ITERATIONS

	FMatrix AX = (FMatrix)0;

#if LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_NEWTON_CHOLESKY
	float MaxTolerance = 1000;
	float CurrentTolerance = 0.0f;
#endif

	for (int Step = 0; Step < NUM_INVERSE_ITERATIONS_DOUBLE; ++Step)
	{
		// A * X is not guranteed to be symmetric
		AX = LMatrixMultiply(AMatrix, X);

		// The norm might increase when combining choleksy and newton while Cholesky is overfitting.
		// Should stop the application if norm increase is detected.
		#if LINEAR_SOLVER_TYPE == LINEAR_SOLVER_TYPE_NEWTON_CHOLESKY && NEWTON_SCHULZ_EARLY_STOP
		MatrixConvergence(AX, CurrentTolerance);
		if (CurrentTolerance > MaxTolerance)
		{
			break;
		}
		MaxTolerance = min(CurrentTolerance, MaxTolerance);
		#endif

		MatrixNegate(AX);

		// 2*I + (-AX) 
		for (int x = 0; x < MATRIX_DIM; ++x)
		{
			MATRIX(AX, x, x) = DFAdd(2.0f, MATRIX(AX, x, x));
		}

		// X ( 2*I - AX) is guranteed to be symmetric
		X = ToLMatrixMultiply(X, AX);
	}

	return X;
}

bool NewtonIterativeSolve(int AOffset, int ASize, int BOffset, float inLambda, RWBuffer<float> Result)
{
	FLMatrix AMatrix = (FLMatrix)0;

	for (int y = 0; y < MATRIX_DIM; ++y)
	{
		for (int x = 0; x <= y; ++x)
		{
			LMatrix(AMatrix, y, x) = DFPromote(A[AOffset + y * MATRIX_DIM + x]);
		}
	}

	bool bComplete = false;

	AMatrix = MatrixInverse(AMatrix, inLambda, bComplete);

	if (!bComplete)
	{
		return bComplete;
	}

	//Calculate approx(A^{-1}) * B.

	for (int i = 0; i < MATRIX_DIM; ++i)
	{
		for (int k = 0; k < BDim.y; ++k)
		{
			FDFScalar Sum = DFPromote(0.0f);
			for (int j = 0; j < MATRIX_DIM; ++j)
			{
				int2 ij = TLMatrixIndex2D(i,j);
				Sum = DFAdd(Sum, DFMultiply(TLMatrix(AMatrix, ij.y, ij.x), DFPromote(B[BOffset + j * BDim.y + k])));
			}

			Result[BOffset + i * BDim.y + k] = DFDemote(Sum);
		}
	}

	return bComplete;
}

