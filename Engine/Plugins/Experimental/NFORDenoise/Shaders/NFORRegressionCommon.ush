// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NFORRegressionCommon.ush
=============================================================================*/

#define LINEAR_SOLVER_TYPE_NEWTON_SCHULZ		0
#define LINEAR_SOLVER_TYPE_CHOLESKY				1
#define LINEAR_SOLVER_TYPE_NEWTON_CHOLESKY		2

#ifndef LINEAR_SOLVER_TYPE
#define LINEAR_SOLVER_TYPE LINEAR_SOLVER_TYPE_NEWTON_SCHULZ
#endif

#define INITIAL_GUESS_INVERSE_CHOLESKY_DECOMPOSITION	0
#define INITIAL_GUESS_EUCLIDEAN_NORM					1

#ifndef NEWTON_INITIAL_GUESS_TYPE
#define NEWTON_INITIAL_GUESS_TYPE INITIAL_GUESS_INVERSE_CHOLESKY_DECOMPOSITION
#endif

#define NEWTON_SCHULZ_EARLY_STOP	1