// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NFORDenoise : ModuleRules
	{
		public NFORDenoise(ReadOnlyTargetRules Target) : base(Target)
		{

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
							"Core",
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RenderCore",
					"Renderer",
					"RHI",
					"Projects",	// Use shader folder remapping
					"Eigen",	// Debug first order regression
				}
			);

		}
	}
}
