// Copyright Epic Games, Inc. All Rights Reserved.

#include "NFORDenoise.h"
#include "Interfaces/IPluginManager.h"
#include "PathTracingDenoiser.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "NFORDenoiseCS.h"

#define LOCTEXT_NAMESPACE "FNFORDenoiseModule"

DEFINE_LOG_CATEGORY(LogNFORDenoise)

using namespace UE::Renderer::Private;

static TAutoConsoleVariable<bool> CVarNFORFeatureDepth(
	TEXT("r.NFOR.Feature.Depth"),
	false,
	TEXT("true: Add depth as feature for NFOR denoising."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarNFORMemoryMode(
	TEXT("r.NFOR.Memory.Mode"),
	1,
	TEXT("The memory mode to use when caching textures.\n")
	TEXT("0: Low memory mode. All 32bit radiance will be cached in 16bit.\n")
	TEXT("1: High memory mode. Use the default texture format used in path tracing."),
	ECVF_RenderThreadSafe
);

bool UseLowMemoryMode()
{
	return CVarNFORMemoryMode.GetValueOnRenderThread() == 0;
}

FRDGTextureDesc ApplyMemoryMode(const FRDGTextureDesc& Desc)
{

	if (!UseLowMemoryMode())
	{
		return Desc;
	}

	FRDGTextureDesc TmpDesc = Desc;

	if (Desc.Format == PF_A32B32G32R32F)
	{
		TmpDesc.Format = PF_FloatRGBA;
	}

	return TmpDesc;
}

bool IsFeatureDepthEnabled()
{
	return CVarNFORFeatureDepth.GetValueOnRenderThread();
}

struct FExternalFrameCache;

struct FRDGFrameCache
{
	FRDGTextureRef Radiance;
	FRDGTextureRef RadianceVariance;
	FRDGTextureRef Albedo;
	FRDGTextureRef Normal;
	FRDGTextureRef Depth;

	FExternalFrameCache ToExternalResource(FRDGBuilder& GraphBuilder, const FSceneView& View);
	void AppendToNFORBuffer(TArray<NFORDenoise::FRadianceDesc>& Radiances, TArray<NFORDenoise::FFeatureDesc>& Features);
};

struct FExternalFrameCache
{
	TRefCountPtr<IPooledRenderTarget> Radiance;
	TRefCountPtr<IPooledRenderTarget> RadianceVariance;
	TRefCountPtr<IPooledRenderTarget> Albedo;
	TRefCountPtr<IPooledRenderTarget> Normal;
	TRefCountPtr<IPooledRenderTarget> Depth;

	FRDGFrameCache ToRGDResource(FRDGBuilder& GraphBuilder, const FSceneView& View);
};

FExternalFrameCache FRDGFrameCache::ToExternalResource(FRDGBuilder& GraphBuilder, const FSceneView& View)
{
	FExternalFrameCache ExternalFrameCache;
	ExternalFrameCache.Radiance = GraphBuilder.ConvertToExternalTexture(Radiance);
	ExternalFrameCache.RadianceVariance = GraphBuilder.ConvertToExternalTexture(RadianceVariance);
	ExternalFrameCache.Albedo = GraphBuilder.ConvertToExternalTexture(Albedo);
	ExternalFrameCache.Normal = GraphBuilder.ConvertToExternalTexture(Normal);
	ExternalFrameCache.Depth = GraphBuilder.ConvertToExternalTexture(Depth);

	return ExternalFrameCache;
}

void FRDGFrameCache::AppendToNFORBuffer(
	TArray<NFORDenoise::FRadianceDesc>& Radiances,
	TArray<NFORDenoise::FFeatureDesc>& Features)
{
	{
		NFORDenoise::FNFORTextureDesc RadianceTex(Radiance, 0, 3, 4);
		NFORDenoise::FNFORTextureDesc RadianceVarianceTex(RadianceVariance, 0, 1, 4);
		Radiances.Add(NFORDenoise::FRadianceDesc(RadianceTex, RadianceVarianceTex, NFORDenoise::EVarianceType::GreyScale));
	}

	{
		NFORDenoise::FNFORTextureDesc AlbedoTex(Albedo, 0, 3, 4);
		NFORDenoise::FNFORTextureDesc AlbedoVarianceTex(RadianceVariance, 1, 1, 4);

		NFORDenoise::FNFORTextureDesc NormalTex(Normal, 0, 3, 4);
		NFORDenoise::FNFORTextureDesc NormalVarianceTex(RadianceVariance, 2, 1, 4);

		Features.Add(NFORDenoise::FFeatureDesc(AlbedoTex, AlbedoVarianceTex, NFORDenoise::EVarianceType::GreyScale));
		Features.Add(NFORDenoise::FFeatureDesc(NormalTex, NormalVarianceTex, NFORDenoise::EVarianceType::Normal));

		if (IsFeatureDepthEnabled())
		{
			NFORDenoise::FNFORTextureDesc DepthTex(Depth, 0, 1, 1);
			NFORDenoise::FNFORTextureDesc DepthVarianceTex(nullptr);
			Features.Add(NFORDenoise::FFeatureDesc(DepthTex, DepthVarianceTex, NFORDenoise::EVarianceType::GreyScale, true));
		}
	}
}

FRDGFrameCache FExternalFrameCache::ToRGDResource(FRDGBuilder& GraphBuilder, const FSceneView& View)
{
	FRDGFrameCache RDGFrameCache;
	RDGFrameCache.Radiance = GraphBuilder.RegisterExternalTexture(Radiance);
	RDGFrameCache.RadianceVariance = GraphBuilder.RegisterExternalTexture(RadianceVariance);
	RDGFrameCache.Albedo = GraphBuilder.RegisterExternalTexture(Albedo);
	RDGFrameCache.Normal = GraphBuilder.RegisterExternalTexture(Normal);
	RDGFrameCache.Depth = GraphBuilder.RegisterExternalTexture(Depth);

	return RDGFrameCache;
}


class FNFORDenoiser : public IPathTracingSpatialTemporalDenoiser
{
public:
	class FHistory : public IHistory
	{
	public:
		FHistory(const TCHAR* DebugName) : DebugName(DebugName) {}

		virtual ~FHistory() 
		{ 
		}

		const TCHAR* GetDebugName() const override { return DebugName; };


		void AddFrame(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs)
		{
			const int NumFrames = NFORDenoise::GetFrameCount(View);
			if (NumFrames <= 0)
			{
				return;
			}

			RDG_EVENT_SCOPE(GraphBuilder, "AddFrame");

			FRDGFrameCache RDGFrameCache;
			
			if (FrameCaches.Num() < NumFrames)
			{
				RDGFrameCache.Radiance = GraphBuilder.CreateTexture(
					ApplyMemoryMode(Inputs.ColorTex->Desc), TEXT("NFOR.Radiance"), ERDGTextureFlags::MultiFrame);
				RDGFrameCache.RadianceVariance = GraphBuilder.CreateTexture(
					ApplyMemoryMode(Inputs.VarianceTex->Desc), TEXT("NFOR.RadianceVariance"), ERDGTextureFlags::MultiFrame);

				RDGFrameCache.Albedo = GraphBuilder.CreateTexture(Inputs.AlbedoTex->Desc, TEXT("NFOR.Albedo"), ERDGTextureFlags::MultiFrame);
				RDGFrameCache.Normal = GraphBuilder.CreateTexture(Inputs.NormalTex->Desc, TEXT("NFOR.Normal"), ERDGTextureFlags::MultiFrame);
				RDGFrameCache.Depth = GraphBuilder.CreateTexture(Inputs.DepthTex->Desc, TEXT("NFOR.Depth"), ERDGTextureFlags::MultiFrame);
			}
			else
			{
				// Remove unreachable caches if Frame count is changed between each denoising cycle.
				while (FrameCaches.Num() > NumFrames)
				{
					FrameCaches.RemoveAt(0);
				}

				// Avoid potential reallocation.
				auto First = MoveTemp(FrameCaches[0]);
				FrameCaches.RemoveAt(0, 1, EAllowShrinking::No);
				FrameCaches.Add(MoveTemp(First));
				RDGFrameCache = FrameCaches[NumFrames - 1].ToRGDResource(GraphBuilder, View);
			}

			if (UseLowMemoryMode())
			{
				// Radiance and radiance variance source and target can have different 4 channel format. Use customized function.
				NFORDenoise::AddCopyMirroredTexturePass(GraphBuilder, Inputs.ColorTex, RDGFrameCache.Radiance);
				NFORDenoise::AddCopyMirroredTexturePass(GraphBuilder, Inputs.VarianceTex, RDGFrameCache.RadianceVariance);
			}
			else
			{
				AddCopyTexturePass(GraphBuilder, Inputs.ColorTex, RDGFrameCache.Radiance);
				AddCopyTexturePass(GraphBuilder, Inputs.VarianceTex, RDGFrameCache.RadianceVariance);
			}

			AddCopyTexturePass(GraphBuilder, Inputs.AlbedoTex, RDGFrameCache.Albedo);
			AddCopyTexturePass(GraphBuilder, Inputs.NormalTex, RDGFrameCache.Normal);
			AddCopyTexturePass(GraphBuilder, Inputs.DepthTex, RDGFrameCache.Depth);
			
			if (FrameCaches.Num() < NumFrames)
			{
				FExternalFrameCache FrameCache = RDGFrameCache.ToExternalResource(GraphBuilder, View);
				FrameCaches.Add(MoveTemp(FrameCache));
			}
		}

		void FetchFrames(FRDGBuilder& GraphBuilder, const FSceneView& View, 
			TArray<NFORDenoise::FRadianceDesc>& Radiances,
			TArray<NFORDenoise::FFeatureDesc>& FeatureDescs,
			bool bReverseOrder = true)
		{
			const int TotalCachedFrames = FrameCaches.Num();
			for (int i = 0; i < TotalCachedFrames; ++i)
			{
				int Index = bReverseOrder ? (TotalCachedFrames - i - 1) : i;
				FExternalFrameCache& FrameCache = FrameCaches[Index];
				FRDGFrameCache RDGFrameCache = FrameCache.ToRGDResource(GraphBuilder, View);
				RDGFrameCache.AppendToNFORBuffer(Radiances, FeatureDescs);
			}
		}

		FRDGTextureRef GetDepth(FRDGBuilder& GraphBuilder, int32 FrameIndex)
		{
			check(FrameIndex >=0 && FrameIndex < Num());
			TRefCountPtr<IPooledRenderTarget> Depth = FrameCaches[FrameIndex].Depth;
			return Depth ? GraphBuilder.RegisterExternalTexture(Depth) : nullptr;
		}

		int32 Num() const { return FrameCaches.Num(); }

	private:
		const TCHAR* DebugName;

		TArray<FExternalFrameCache> FrameCaches;
	};

	~FNFORDenoiser() {}

	const TCHAR* GetDebugName() const override { return *DebugName; }

	virtual bool NeedVarianceTexture() const override { return true;}

	FOutputs AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const override
	{
		AddCopyTexturePass(GraphBuilder,Inputs.ColorTex, Inputs.OutputTex);

		FHistory* History = Inputs.PrevHistory.IsValid() ? reinterpret_cast<FHistory*>(Inputs.PrevHistory.GetReference()) : nullptr;

		if (!Inputs.VarianceTex)
		{
			return { History };
		}
		
		TUniquePtr<FHistory> CurHistory = MakeUnique<FHistory>(*DebugName);
		if (History)
		{
			CurHistory.Reset(History);
		}

		CurHistory->AddFrame(GraphBuilder, View, Inputs);
		
		// Fetch all the frames for denoising
		TArray<NFORDenoise::FFeatureDesc> Features;
		TArray<NFORDenoise::FRadianceDesc> Radiances;

		CurHistory->FetchFrames(GraphBuilder, View, Radiances, Features);

		// Denoising based on spatial temporal denoising config
		const bool bDenoised = NFORDenoise::FilterMain(
				GraphBuilder,
				View,
				Radiances,
				Features, 
				Inputs.OutputTex);

		UE_LOGF(LogNFORDenoise, Log, "Frame %d: %ls NumOfHistory = %d", Inputs.DenoisingFrameId, bDenoised ? TEXT("Denoised") : TEXT("Accumulating history"), CurHistory ? CurHistory->Num() : 0);

		int32 NumberOfHistory = CurHistory->Num();

		// Post process DOF replies on the depth buffer to estimate the DOF,
		// Update the output depth buffer to the depth of the center image if the denoising frame is not the latest.
		const bool bNeedToUpdateDepth = bDenoised && (NFORDenoise::GetFrameCount(View) > 1) && NFORDenoise::GetDenoisingFrameIndex(View, NumberOfHistory) > 0;
		if (bNeedToUpdateDepth)
		{
				
			int32 DenoisingDepthFrameIndex = NumberOfHistory - NFORDenoise::GetDenoisingFrameIndex(View,NumberOfHistory) - 1;
			if (DenoisingDepthFrameIndex >= 0)
			{
				const bool bAlphaOnly = true;
				FRDGTextureRef DenoisingDepth = CurHistory->GetDepth(GraphBuilder, DenoisingDepthFrameIndex);

				AddCopyTexturePass(GraphBuilder, DenoisingDepth, Inputs.DepthTex);
			}
		}

		// TODO: Output denoised albedo, normal if needed.
		return { CurHistory.Release() };
	}

	void AddMotionVectorPass(FRDGBuilder& GraphBuilder, const FSceneView& View, const FMotionVectorInputs& Inputs) const override
	{
		AddClearRenderTargetPass(GraphBuilder, Inputs.OutputTex, FLinearColor::Black);
	}

private:
	inline static const FString DebugName = TEXT("FNFORDenoiser");
};

void FNFORDenoiseModule::StartupModule()
{
	UE_LOGF(LogNFORDenoise, Log, "NFORDenoise function starting up");

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("NFORDenoise"));
	if (Plugin.IsValid())
	{
		FString ModuleDir = Plugin->GetBaseDir();
		AddShaderSourceDirectoryMapping(TEXT("/NFORDenoise"), FPaths::Combine(ModuleDir, TEXT("/Shaders")));

		RegisterSpatialTemporalDenoiser(MakeUnique<FNFORDenoiser>(), TEXT("NFOR"));
	}
	else
	{
		UE_LOGF(LogNFORDenoise, Error, "Shaders directory not added. Failed to find NFORDenoise plugin");
	}
}

void FNFORDenoiseModule::ShutdownModule()
{
	UE_LOGF(LogNFORDenoise, Log, "NFORDenoise function shutting down");
	UnregisterDenoiser(TEXT("NFOR"));
}

#undef LOCTEXT_NAMESPACE
IMPLEMENT_MODULE(FNFORDenoiseModule, NFORDenoise)