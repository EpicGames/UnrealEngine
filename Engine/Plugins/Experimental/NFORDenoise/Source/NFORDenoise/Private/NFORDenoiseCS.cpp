// Copyright Epic Games, Inc. All Rights Reserved.

#include "NFORDenoiseCS.h"
#include "SystemTextures.h"
#include "PixelShaderUtils.h"
#include "SceneView.h"

#include "NFORWeightedLSRCommon.h"
#include "NFORRegressionCPUSolver.h"

namespace NFORDenoise
{
	
	TAutoConsoleVariable<bool> CVarNFORFeatureAddConstant(
		TEXT("r.NFOR.Feature.AddConstant"),
		1,
		TEXT("Add a constant 1 feature for denoising. Especially useful when all other features are zero."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORFeatureMaxAlbedoGreyscale(
		TEXT("r.NFOR.Feature.MaxAlbedoGreyscale"),
		2.0,
		TEXT("Set the max albedo in greyscale used for denoising. Scale the albedo variance as well. Used for suppressing specular noise.")
		TEXT("<=0: Ignore scaling."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORFeatureMaxNormalLength(
		TEXT("r.NFOR.Feature.MaxNormalLength"),
		10.0,
		TEXT("Set the max normal length used for denoising. Scale the normal variance as well. Used for suppressing specular noise.")
		TEXT("<=0: Ignore scaling."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarNFORFeatureFiltering(
		TEXT("r.NFOR.Feature.Filtering"),
		true,
		TEXT("True: Filter all features.\n")
		TEXT("False: Disable feature filtering (useful for debug).\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORPredivideAlbedo(
		TEXT("r.NFOR.PredivideAlbedo"),
		1,
		TEXT("Enable pre-albedo divide to denoise only the demodulated singal. It preserves more high frequency details."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORPredivideAlbedoOffset(
		TEXT("r.NFOR.PredivideAlbedo.Offset"),
		0.1f,
		TEXT("Offset for albedo for regions other than full reflection and sky materials. Increase to get a smoother result."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORPredivideAlbedoOffsetSky(
		TEXT("r.NFOR.PredivideAlbedo.OffsetSky"),
		0.2f,
		TEXT("Sky or reflection of sky material has very small albedo that will cause noise. Offset more."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORFrameCount(
		TEXT("r.NFOR.FrameCount"),
		2,
		TEXT("n: Use the previous n frames, the current frame, and the future n frames. Suggested range is 0~2. Max=3.(Offline config)\n")
		TEXT("The value is always 0 for online preview denoising\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORFrameCountCondition(
		TEXT("r.NFOR.FrameCount.Condition"),
		1,
		TEXT("0: Denoise even if the frame count accumulated is less than the required frame count (used for debug).")
		TEXT("1: Denoise only when the number of frame count meets requirement."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORDenoisingFrameIndex(
		TEXT("r.NFOR.DenoisingFrameIndex"),
		-1,
		TEXT("The index of the denoising frame.")
		TEXT("-1: Automatically determine the index.")
		TEXT("i: Use all frames other than the ith frame to denoise."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNumOfTile(
		TEXT("r.NFOR.NumOfTile"),
		-1,
		TEXT("n: Divide the image into n x n tiles in [1,32].\n")
		TEXT("0<=x<=1: Use a single dispatch. Could run out of memory.\n")
		TEXT("-1: Automatically determine the number of tiles based on r.NFOR.Tile.Size and the view size.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORTileSize(
		TEXT("r.NFOR.Tile.Size"),
		213,
		TEXT("The size of the max length of a tile. The default is selected for best performance based on experiment.\n")
		TEXT("It takes effect only when r.NFOR.NumOfTile is set to -1. Minimal value = 100.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORTileFeatureTileCountDownScale(
		TEXT("r.NFOR.Tile.FeatureTileCount.DownScale"),
		2,
		TEXT("Reduce the number of tiles by this factor for feature filtering."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORTileDebug(
		TEXT("r.NFOR.TileDebug"),
		0,
		TEXT(">0: Turn on tile debug mode."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORTileDebugIndex(
		TEXT("r.NFOR.TileDebug.Index"),
		-1,
		TEXT("Tile index number to debug.")
		TEXT(" -1: The middle index in range of 0 ~ (NumOfTile * NumOfTile - 1).")
		TEXT(">=0: Select a specific tile to render for debug."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORRegressionDevice(
		TEXT("r.NFOR.Regression.Device"),
		1,
		TEXT(" 0: CPU (verification). Used only for feature development.\n")
		TEXT(" 1: GPU.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORRegressionDataRatioToParameters(
		TEXT("r.NFOR.Regression.MaxDataRatioToParemters"),
		20.0f,
		TEXT("The max number of observations per parameter in the regression. <1 to use all."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORLinearSolverDevice(
		TEXT("r.NFOR.LinearSolver.Device"),
		1,
		TEXT("0: Solve Ax=B on CPU. Use householder QR decomposition from Eigen library.")
		TEXT("1: Solve Ax=B on GPU."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORLinearSolverType(
		TEXT("r.NFOR.LinearSolver.Type"),
		0,
		TEXT("The linear regression solver type implemented in GPU.\n")
		TEXT("0: Newton Schulz iterative method (High quality but slow).\n")
		TEXT("1: Cholesky decomposition (Fast but has too smoothed result or artifacts).\n")
		TEXT("2: Fusion of Cholesky and Newton Schulz iterative method (High quality and fast).\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORLinearSolverCholeskyLambda(
		TEXT("r.NFOR.LinearSolver.Cholesky.Lambda"),
		2e-5f,
		TEXT("The initial lambda for modified Cholesky decomposition to make it positive definite. It will be scaled by the max of the absolute of the matrix element.\n")
		TEXT("Large value yields bias with smoothed rendering, while small value leads to variance or artifacts.\n")
		TEXT("Used when r.NFOR.LinearSolver.Type = 1 and 2. Selected to match the quality of r.NFOR.LinearSolver.Type = 2\n")
		TEXT("to r.NFOR.LinearSolver.Type 0."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORReconstructionType(
		TEXT("r.NFOR.Reconstruction.Type"),
		0,
		TEXT("0: Scatter for the denoising frame, gather for other temporal frames (default).")
		TEXT("1: Force gathering."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORReconstructionDebugFrameIndex(
		TEXT("r.NFOR.Reconstruction.Debug.FrameIndex"),
		-1,
		TEXT(">=0: Output the denoising contribution from the ith frame only.")
		TEXT("-1: do not perform debug. Output contributions from all frames."),
		ECVF_RenderThreadSafe);
	
	TAutoConsoleVariable<bool> CVarNFORNonLocalMeanAtlas(
		TEXT("r.NFOR.NonLocalMean.Atlas"),
		true,
		TEXT("true	: Use atlas and separable filter to improve the performance of NLM weights query.\n")
		TEXT("false : Calculate the non local mean weights for each pixel in place.(baseline but slow).\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanAtlasType(
		TEXT("r.NFOR.NonLocalMean.Atlas.Type"),
		1,
		TEXT("0: float2. Stores one symmetric distance/weight.\n")
		TEXT("1: float4. Stores two symmetric distance/weights.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanAtlasSize(
		TEXT("r.NFOR.NonLocalMean.Atlas.Size"),
		2048,
		TEXT("<=0: Use the same size of the input tile.\n")
		TEXT("	n: At least the max size of the input tile(2k as default). The larger, the less number of dispatch passes.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanWeightLayout(
		TEXT("r.NFOR.NonLocalMean.WeightLayout"),
		3,
		TEXT("The layout of the weight. It affects the performance and how weights are handled at each stage.\n")
		TEXT(" 0: Do not use weight buffer.\n")
		TEXT(" 1: NumOfWeightsPerPixel x Width x Height.\n")
		TEXT(" 2: Width x Height x NumOfWeightsPerPixel.\n")
		TEXT(" 3: float4 x Width x Height x DivideAndRoundUp(NumOfWeightsPerPixel,float4)\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNOnLocalMeanFeatureFormat(
		TEXT("r.NFOR.NonLocalMean.Feature.Format"),
		1,
		TEXT("0: fp32 \n")
		TEXT("1: fp16 \n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanFeaturePatchSize(
		TEXT("r.NFOR.NonLocalMean.Feature.PatchSize"),
		3,
		TEXT("The patch size of the non-local mean algorithm for feature filtering. The patch width/height = PatchSize * 2 + 1."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanFeaturePatchDistance(
		TEXT("r.NFOR.NonLocalMean.Feature.PatchDistance"),
		5,
		TEXT("The search distance of the non-local mean algorithm for feature filtering. The searching patch width/height = PatchDistance * 2 + 1."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanRadiancePatchSize(
		TEXT("r.NFOR.NonLocalMean.Radiance.PatchSize"),
		3,
		TEXT("The patch size of the non-local mean algorithm. The patch width/height = PatchSize * 2 + 1."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarNFORNonLocalMeanRadiancePatchDistance(
		TEXT("r.NFOR.NonLocalMean.Radiance.PatchDistance"),
		9,
		TEXT("The search distance of the non-local mean algorithm. The searching patch width/height = PatchDistance * 2 + 1.")
		TEXT("The patch distance for bandwidth selection dependents on this parameters for MSE and selection filtering."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarNFORDenoiseAlpha(
		TEXT("r.NFOR.Denoise.Alpha"),
		true,
		TEXT("Indicate if the alpha channel of radiance will be denoised (Default on)."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarNFORBandwidthSelection(
		TEXT("r.NFOR.BandwidthSelection"),
		true,
		TEXT("true: Apply bandwidth selection. It helps to preserve both high and low frequency details."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarNFORBandwidthSelectionBandwidth(
		TEXT("r.NFOR.BandwidthSelection.Bandwidth"),
		-1,
		TEXT("-1: Use predefined bandwidths {0.5f, 1.0f}.")
		TEXT("(0,1]: Use a specific bandwidth."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarNFORBandwidthSelectionMSEPreserveDetail(
		TEXT("r.NFOR.BandwidthSelection.MSE.PreserveDetail"),
		true,
		TEXT("false: Use bandwidth = 1.0 to filter MSE.")
		TEXT("true: Use the corresponding bandwidth to filter MSE"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarNFORBandwidthSelectionMapPreserveDetail(
		TEXT("r.NFOR.BandwidthSelection.Map.PreserveDetail"),
		false,
		TEXT("false: Use bandwidth = 1.0 to filter MSE.")
		TEXT("true: Use the corresponding bandwidth to filter MSE"),
		ECVF_RenderThreadSafe);

	// Working in progress
	TAutoConsoleVariable<int32> CVarNFORAlbedoDivideRecoverPhase(
		TEXT("r.NFOR.AlbedoDivide.RecoverPhase"),
		0,
		TEXT("0: Add back in the last step. Denoised = Albedo_{center} * \\sum_{all frames}{denoised radiance}. Require high sample count for high quality albedo.")
		TEXT("1: Add back at each scattering or gathering. Denoised = * \\sum_{i \\in frames}{Albedo_i * denoised radiance}."),
		ECVF_RenderThreadSafe
	);

	//--------------------------------------------------------------------------------------------------------------------
	// Functions based on CVars

	bool ShouldCompileNFORShadersForProject(EShaderPlatform ShaderPlatform)
	{
		static const IConsoleVariable* CVarPathTracing = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PathTracing"));
		const bool bSupportsPathTracing = CVarPathTracing ? CVarPathTracing->GetInt() != 0 : false;

		return ShouldCompileRayTracingShadersForProject(ShaderPlatform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(ShaderPlatform) &&
			bSupportsPathTracing &&
			!IsMetalPlatform(ShaderPlatform);
	}

	bool ShouldFeatureAddConstant()
	{
		return CVarNFORFeatureAddConstant.GetValueOnRenderThread();
	}

	float GetFeatureMaxAlbedoGrayscale()
	{
		return CVarNFORFeatureMaxAlbedoGreyscale.GetValueOnRenderThread();
	}

	float GetFeatureMaxNormalLength()
	{
		return CVarNFORFeatureMaxNormalLength.GetValueOnRenderThread();
	}

	bool ShouldApplyFeatureFiltering()
	{
		return CVarNFORFeatureFiltering.GetValueOnRenderThread();
	}

	bool IsPreAlbedoDivideEnabled()
	{
		const bool bPredivideAlbedo = CVarNFORPredivideAlbedo.GetValueOnRenderThread() != 0;
		return bPredivideAlbedo;
	}

	FLinearColor GetPreAlbedoDivideAlbedoOffset()
	{
		const float Offset = FMath::Max(1e-8, CVarNFORPredivideAlbedoOffset.GetValueOnRenderThread());
		const float OffsetSky = FMath::Max(1e-8, CVarNFORPredivideAlbedoOffsetSky.GetValueOnRenderThread());
		return FLinearColor(Offset, OffsetSky, 0.0f);
	}

	int32 GetFrameCount(const FSceneView& View)
	{
		int32 NumFrames = FMath::Clamp(1 + 2 * CVarNFORFrameCount.GetValueOnRenderThread(), 1, 7);
		if (!View.bIsOfflineRender)
		{
			NumFrames = 1;
		}
		return NumFrames;
	}

	enum class EDenoiseFrameCountCondition : uint32
	{
		Any,
		Equal,
		MAX
	};

	EDenoiseFrameCountCondition GetFrameCountCoundition()
	{
		uint32 ConditionValue = CVarNFORFrameCountCondition.GetValueOnRenderThread();
		EDenoiseFrameCountCondition Condition = EDenoiseFrameCountCondition::Any;
		if (ConditionValue != 0)
		{
			Condition = EDenoiseFrameCountCondition::Equal;
		}
		return Condition;
	}

	int32 GetDenoisingFrameIndex(const FSceneView& View, int32 NumberOfFrameInBuffer)
	{
		int32 TargetFrameCount = GetFrameCount(View);
		int32 DenoisingFrameIndex = CVarNFORDenoisingFrameIndex.GetValueOnRenderThread();
		int32 ResolvedSourceFrameIndex = INDEX_NONE;
		if (DenoisingFrameIndex < 0)
		{
			// If no specific denoising frame index is specified, use the center one
			if (NumberOfFrameInBuffer >= 0)
			{
				ResolvedSourceFrameIndex = (NumberOfFrameInBuffer > TargetFrameCount / 2) ? (TargetFrameCount / 2) : INDEX_NONE;
			}
			else
			{
				ResolvedSourceFrameIndex = TargetFrameCount / 2;
			}
		}
		else
		{
			// If the user has set the denoising index, use the available index within the limit
			ResolvedSourceFrameIndex = FMath::Clamp(DenoisingFrameIndex, 0, TargetFrameCount - 1);
			if (NumberOfFrameInBuffer - 1 < ResolvedSourceFrameIndex)
			{
				ResolvedSourceFrameIndex = INDEX_NONE;
			}
		}
		
		return ResolvedSourceFrameIndex;
	}

	int32 GetNumOfTiles(FIntPoint TextureSize)
	{
		int32 NumOfTile = CVarNFORNumOfTile.GetValueOnRenderThread();
		if (NumOfTile < 0)
		{
			// If the max texture size is 1920, and tile size is 192, the num of tiles is 10x10.
			// If it is between 1920 and 2111, it remains the same until it becames 2112, the tiles will be 11x11.
			const int32 MaxTextureSize = TextureSize.GetMax();
			const int32 TileSize = FMath::Max(CVarNFORTileSize.GetValueOnRenderThread(), 100);

			NumOfTile = MaxTextureSize / TileSize;
		}
		return FMath::Clamp(NumOfTile, 1, 32);
	}

	int32 GetFeatureTileSizeDownScale()
	{
		return FMath::Max(CVarNFORTileFeatureTileCountDownScale.GetValueOnRenderThread(), 1);
	}

	bool IsTileDebugEnabled()
	{
		return CVarNFORTileDebug.GetValueOnRenderThread() > 0;
	}

	int32 GetTileDebugIndex()
	{
		return CVarNFORTileDebugIndex.GetValueOnRenderThread();
	}

	enum class ERegressionDevice : int32
	{
		CPU,
		GPU,
		MAX
	};

	ERegressionDevice GetRegressionDevice()
	{
		const int32 RegressionDevice = FMath::Clamp(CVarNFORRegressionDevice.GetValueOnRenderThread(),
			static_cast<int32>(ERegressionDevice::CPU),
			static_cast<int32>(ERegressionDevice::MAX) - 1);
		return static_cast<ERegressionDevice>(RegressionDevice);
	}

	int32 GetSamplingStep(int32 NumberOfParameters, int32 TotalDataRecords)
	{
		int32 DataRatioToParameters = CVarNFORRegressionDataRatioToParameters.GetValueOnRenderThread();
		if (DataRatioToParameters < 1)
		{
			return 1;
		}
		return FMath::Max(1, TotalDataRecords / (NumberOfParameters * DataRatioToParameters));
	}

	enum class ELinearSolverDevice : int32
	{
		CPU,
		GPU,
		MAX
	};

	ELinearSolverDevice GetLinearSolverDevice()
	{
		const int32 LinearSolverDevice = FMath::Clamp(CVarNFORLinearSolverDevice.GetValueOnRenderThread(),
			static_cast<int32>(ELinearSolverDevice::CPU),
			static_cast<int32>(ELinearSolverDevice::MAX) - 1);
		return static_cast<ELinearSolverDevice>(LinearSolverDevice);
	}

	RegressionKernel::FLinearSolverCS::ESolverType GetLinearSolverType()
	{
		const int32 LinearSolverType = FMath::Clamp(CVarNFORLinearSolverType.GetValueOnRenderThread(),
			0,
			static_cast<int32>(RegressionKernel::FLinearSolverCS::ESolverType::MAX));
		// MAX indicates using fusion.
		return static_cast<RegressionKernel::FLinearSolverCS::ESolverType>(LinearSolverType);
	}

	const TCHAR* GetLinearSolverTypeName(RegressionKernel::FLinearSolverCS::ESolverType& SolverType)
	{
		static const TCHAR* const kEventNames[] = {
				TEXT("NewtonSchulz"),
				TEXT("Cholesky"),
				TEXT("NewtonCholesky"),
				TEXT("Fusion")
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == (int32(RegressionKernel::FLinearSolverCS::ESolverType::MAX)+1), "Fix me");
		return kEventNames[int32(SolverType)];
	}

	float GetLinearSolverCholeskyLambda()
	{
		return FMath::Max(CVarNFORLinearSolverCholeskyLambda.GetValueOnRenderThread(),0);
	}

	RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType 
		GetReconstructionType( int32 CurrentFrameIndex, int32 DenoisingFrameIndex)
	{
		RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType ReconstructionType;
		ReconstructionType = RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType::Gather;

		if (CurrentFrameIndex == DenoisingFrameIndex)
		{
			ReconstructionType = RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType::Scatter;
		}

		if (CVarNFORReconstructionType.GetValueOnRenderThread() != 0)
		{
			ReconstructionType = RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType::Gather;
		}

		return ReconstructionType;
	}

	int32 GetReconstructionDebugFrameIndex()
	{
		return CVarNFORReconstructionDebugFrameIndex.GetValueOnRenderThread();
	}

	int32 GetNonLocalMeanFeaturePatchSize()
	{
		return FMath::Clamp(CVarNFORNonLocalMeanFeaturePatchSize.GetValueOnRenderThread(), 0, 10);
	}

	int32 GetNonLocalMeanFeaturePatchDistance()
	{
		return FMath::Clamp(CVarNFORNonLocalMeanFeaturePatchDistance.GetValueOnRenderThread(), 0, 30);
	}

	int32 GetNonLocalMeanRadiancePatchSize()
	{
		return FMath::Clamp(CVarNFORNonLocalMeanRadiancePatchSize.GetValueOnRenderThread(), 0, 10);
	}

	int32 GetNonLocalMeanRadiancePatchDistance()
	{
		return FMath::Clamp(CVarNFORNonLocalMeanRadiancePatchDistance.GetValueOnRenderThread(), 0, 30);
	}

	ENonLocalMeanWeightLayout GetNonLocalMeanWeightLayout()
	{
		const uint32 LayoutType = FMath::Clamp(
			CVarNFORNonLocalMeanWeightLayout.GetValueOnRenderThread(),
			0,
			static_cast<uint32>(ENonLocalMeanWeightLayout::MAX) - 1
		);
		return static_cast<ENonLocalMeanWeightLayout>(LayoutType);
	}

	int32 GetNonLocalMeanSingleFrameWeightBufferSize(FIntPoint Size, int32 NumOfWeightsPerPixel)
	{
		ENonLocalMeanWeightLayout NonLocalMeanWeightLayout = GetNonLocalMeanWeightLayout();
		int32 NumOfElements = Size.X * Size.Y * NumOfWeightsPerPixel;
		if (NonLocalMeanWeightLayout == ENonLocalMeanWeightLayout::Float4xWxHxNumOfWeightsPerPixelByFloat4)
		{
			NumOfElements = Size.X * Size.Y * FMath::DivideAndRoundUp(NumOfWeightsPerPixel, 4) * 4;
		}
		return NumOfElements;
	}

	const TCHAR* GetNonLocalMeanWeightLayoutName(ENonLocalMeanWeightLayout WeightLayout)
	{
		static const TCHAR* const kEventNames[] = {
				TEXT("Direct"),
				TEXT("Buffer(WeightxWxH)"),
				TEXT("Buffer(WxHxWeight)"),
				TEXT("Buffer(4xWxHx[Weight/4])")
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(ENonLocalMeanWeightLayout::MAX), "Fix me");
		return kEventNames[int32(WeightLayout)];
	}

	FIntPoint GetNonLocalMeanAtlasSize(FIntPoint Extent)
	{
		int32 AtlasSize = CVarNFORNonLocalMeanAtlasSize.GetValueOnRenderThread();
		if (AtlasSize <= 0)
		{
			return Extent;
		}
		else
		{
			AtlasSize = FMath::Max(Extent.GetMax(),AtlasSize);
			return FIntPoint(AtlasSize,AtlasSize);
		}
	}

	bool ShouldNonLocalMeanUseAtlas()
	{
		return CVarNFORNonLocalMeanAtlas.GetValueOnRenderThread();
	}

	ENonLocalMeanAtlasType GetNonLocalMeanAtlasType()
	{
		const uint32 AtlasType = FMath::Clamp(
			CVarNFORNonLocalMeanAtlasType.GetValueOnRenderThread(),
			0,
			static_cast<uint32>(ENonLocalMeanAtlasType::MAX)-1
			);
		ENonLocalMeanAtlasType NonLocalMeanAtlasType = static_cast<ENonLocalMeanAtlasType>(AtlasType);

		if (GetNonLocalMeanWeightLayout() == ENonLocalMeanWeightLayout::Float4xWxHxNumOfWeightsPerPixelByFloat4)
		{
			NonLocalMeanAtlasType = ENonLocalMeanAtlasType::TwoSymmetricPair;
		}
		return NonLocalMeanAtlasType;
	}

	bool ShouldDenoiseAlpha()
	{
		return CVarNFORDenoiseAlpha.GetValueOnRenderThread();
	}

	bool IsBandwidthSelectionEnabled()
	{
		return CVarNFORBandwidthSelection.GetValueOnRenderThread();
	}

	TArray<float> GetBandwidthsConfiguration()
	{
		TArray<float> Bandwidths = { 0.5f, 1.0f };
		{
			float BandWidthOverride = FMath::Min(CVarNFORBandwidthSelectionBandwidth.GetValueOnRenderThread(), 1.0f);
			if (BandWidthOverride > 0)
			{
				Bandwidths = { BandWidthOverride };
			}
		}

		return Bandwidths;
	}

	bool ShouldBandwidthSelectionMSEPreserveDetail()
	{
		return CVarNFORBandwidthSelectionMSEPreserveDetail.GetValueOnRenderThread();
	}

	bool ShouldBandwidthSelectionMapPreserveDetail()
	{
		return CVarNFORBandwidthSelectionMapPreserveDetail.GetValueOnRenderThread();
	}


	EPixelFormat GetFeaturePixelFormat()
	{
		EPixelFormat PixelFormat = PF_R32_FLOAT;

		if (GetRegressionDevice() != ERegressionDevice::CPU)
		{
			if (CVarNFORNOnLocalMeanFeatureFormat.GetValueOnRenderThread() != 0)
			{
				PixelFormat = EPixelFormat::PF_R16F;
			}
		}
		return PixelFormat;
	}

	uint32 GetFeatureBytesPerElement()
	{
		uint32 ByteSize = sizeof(float);
		if (GetRegressionDevice() != ERegressionDevice::CPU)
		{
			if (CVarNFORNOnLocalMeanFeatureFormat.GetValueOnRenderThread() != 0)
			{
				ByteSize = sizeof(int16_t);
			}
		}

		return ByteSize;
	}

	EAlbedoDivideRecoverPhase GetPreAlbedoDivideRecoverPhase()
	{
		EAlbedoDivideRecoverPhase AlbedoDivideRecoverPhase = EAlbedoDivideRecoverPhase::Disabled;
		if (IsPreAlbedoDivideEnabled())
		{
			if (CVarNFORAlbedoDivideRecoverPhase.GetValueOnRenderThread() == 0)
			{
				AlbedoDivideRecoverPhase = EAlbedoDivideRecoverPhase::Final;
			}
			else
			{
				AlbedoDivideRecoverPhase = EAlbedoDivideRecoverPhase::Each;
			}
		}
		return AlbedoDivideRecoverPhase;
	}

	//--------------------------------------------------------------------------------------------------------------------
	// Shader implementations
	//--------------------------------------------------------------------------------------------------------------------
	// General texture operations
	IMPLEMENT_GLOBAL_SHADER(FTextureMultiplyCS, "/NFORDenoise/NFORDenoise.usf", "TextureOperationCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FTextureDivideCS, "/NFORDenoise/NFORDenoise.usf", "TextureOperationCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FTextureAccumulateConstantCS, "/NFORDenoise/NFORDenoise.usf", "TextureOperationCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FTextureAccumulateCS, "/NFORDenoise/NFORDenoise.usf", "TextureOperationCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FCopyTexturePS, "/NFORDenoise/NFORDenoise.usf", "CopyTexturePS", SF_Pixel);
	IMPLEMENT_GLOBAL_SHADER(FCopyTextureSingleChannelCS, "/NFORDenoise/NFORDenoise.usf", "CopyTextureSingleChannelCS", SF_Compute);

	//--------------------------------------------------------------------------------------------------------------------
	// Feature range adjustment and radiance normalization
	IMPLEMENT_GLOBAL_SHADER(FClassifyPreAlbedoDivideMaskIdCS, "/NFORDenoise/NFORDenoise.usf", "ClassifyPreAlbedoDivideMaskIdCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FNormalizeRadianceVarianceByAlbedoCS, "/NFORDenoise/NFORDenoise.usf", "NormalizeRadianceVarianceByAlbedoCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FAdjustFeatureRangeCS, "/NFORDenoise/NFORDenoise.usf", "AdjustFeatureRangeCS", SF_Compute);

	//--------------------------------------------------------------------------------------------------------------------
	// Non-local mean weight and filtering
	IMPLEMENT_GLOBAL_SHADER(FNonLocalMeanFilteringCS, "/NFORDenoise/NFORDenoise.usf", "NonLocalMeanFilteringCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FNonLocalMeanWeightsCS, "/NFORDenoise/NFORDenoise.usf", "NonLocalMeanWeightsCS", SF_Compute);

	// Fast weights query.
	IMPLEMENT_GLOBAL_SHADER(FNonLocalMeanGetSqauredDistanceToAtlasCS, "/NFORDenoise/NFORDenoise.usf", "NonLocalMeanGetSqauredDistanceToAtlasCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FNonLocalMeanSeperableFilterPatchSqauredDistanceCS, "/NFORDenoise/NFORDenoise.usf", "NonLocalMeanSeperableFilterPatchSqauredDistanceCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FNonLocalMeanReshapeBufferCS, "/NFORDenoise/NFORDenoise.usf", "NonLocalMeanReshapeBufferCS", SF_Compute);

	//--------------------------------------------------------------------------------------------------------------------
	// Collaborative filtering
	//	1. Tiling
	IMPLEMENT_GLOBAL_SHADER(FCopyTextureToBufferCS, "/NFORDenoise/NFORDenoise.usf", "CopyTextureToBufferCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FNormalizeTextureCS, "/NFORDenoise/NFORDenoise.usf", "NormalizeTextureCS", SF_Compute);
	
	//	2. Weighted Least-square solver
	IMPLEMENT_GLOBAL_SHADER(RegressionKernel::FInPlaceBatchedMatrixMultiplicationCS, "/NFORDenoise/NFORDenoise.usf", "InPlaceBatchedMatrixMultiplicationCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(RegressionKernel::FLinearSolverCS, "/NFORDenoise/NFORDenoise.usf", "LinearSolverCS", SF_Compute);
	
	//		Allow quality and speed balance
	IMPLEMENT_GLOBAL_SHADER(RegressionKernel::FLinearSolverBuildIndirectDispatchArgsCS, "/NFORDenoise/NFORDenoise.usf", "LinearSolverBuildIndirectDispatchArgsCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(RegressionKernel::FLinearSolverIndirectCS, "/NFORDenoise/NFORDenoise.usf", "LinearSolverIndirectCS", SF_Compute);

	
	IMPLEMENT_GLOBAL_SHADER(RegressionKernel::FReconstructSpatialTemporalImage, "/NFORDenoise/NFORDenoise.usf", "ReconstructSpatialTemporalImageCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FAccumulateBufferToTextureCS, "/NFORDenoise/NFORDenoise.usf", "AccumulateBufferToTextureCS", SF_Compute);

	//--------------------------------------------------------------------------------------------------------------------
	// Bandwidth selection
	IMPLEMENT_GLOBAL_SHADER(FMSEEstimationCS, "/NFORDenoise/NFORDenoise.usf", "MSEEstimationCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FGenerateSelectionMapCS, "/NFORDenoise/NFORDenoise.usf", "GenerateSelectionMapCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FCombineFilteredImageCS, "/NFORDenoise/NFORDenoise.usf", "CombineFilteredImageCS", SF_Compute);

	//--------------------------------------------------------------------------------------------------------------------
	// General texture operations
	void AddMultiplyTextureRegionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		bool bForceMultiply, FIntPoint SourcePosition, FIntPoint TargetPosition, FIntPoint Size)
	{
		Size = Size == FIntPoint::ZeroValue ? SourceTexture->Desc.Extent : Size;
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		typedef FTextureMultiplyCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->Source = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTexture));
			PassParameters->RWTarget = GraphBuilder.CreateUAV(TargetTexture);
			PassParameters->SourcePosition = SourcePosition;
			PassParameters->TargetPosition = TargetPosition;
			PassParameters->ForceOperation = static_cast<int32>(bForceMultiply);
			PassParameters->Size = Size;
		}

		TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::AddMultiplyTextureRegionPass (%s [%d,%d] -> %s [%d,%d], size:%dx%d)",
				SourceTexture->Name,
				SourcePosition.X,
				SourcePosition.Y,
				TargetTexture->Name,
				TargetPosition.X,
				TargetPosition.Y,
				Size.X,
				Size.Y),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Size, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	void AddDivideTextureRegionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		bool bForceDivide, FIntPoint SourcePosition, FIntPoint TargetPosition, FIntPoint Size)
	{
		Size = Size == FIntPoint::ZeroValue ? SourceTexture->Desc.Extent : Size;
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		typedef FTextureDivideCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->Source = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTexture));
			PassParameters->RWTarget = GraphBuilder.CreateUAV(TargetTexture);
			PassParameters->SourcePosition = SourcePosition;
			PassParameters->TargetPosition = TargetPosition;
			PassParameters->ForceOperation = static_cast<int32>(bForceDivide);
			PassParameters->Size = Size;
		}

		TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::AddDivideTextureRegionPass (%s [%d,%d] -> %s [%d,%d], size:%dx%d)",
				SourceTexture->Name,
				SourcePosition.X,
				SourcePosition.Y,
				TargetTexture->Name,
				TargetPosition.X,
				TargetPosition.Y,
				Size.X,
				Size.Y),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Size, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	void AddAccumulateTextureRegionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		FIntPoint SourcePosition, FIntPoint TargetPosition, FIntPoint Size)
	{
		Size = Size == FIntPoint::ZeroValue ? SourceTexture->Desc.Extent : Size;
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		{
			typedef FTextureAccumulateCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->Source = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTexture));
				PassParameters->RWTarget = GraphBuilder.CreateUAV(TargetTexture);
				PassParameters->SourcePosition = SourcePosition;
				PassParameters->TargetPosition = TargetPosition;
				PassParameters->Size = Size;
			}

			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::AddAccumulateTextureRegionPass (%s [%d,%d] -> %s [%d,%d], size:%dx%d)",
					SourceTexture->Name,
					SourcePosition.X,
					SourcePosition.Y,
					TargetTexture->Name,
					TargetPosition.X,
					TargetPosition.Y,
					Size.X,
					Size.Y),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(Size, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}
	}

	void AddAccumulateConstantRegionPass(FRDGBuilder& GraphBuilder, const FLinearColor& ConstantValue, const FRDGTextureRef& TargetTexture, const FRDGTextureRef& Mask,
		FIntPoint SourcePosition, FIntPoint TargetPosition, FIntPoint Size)
	{
		Size = Size == FIntPoint::ZeroValue ? TargetTexture->Desc.Extent : Size;
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		const bool bUseMask = Mask != nullptr;

		typedef FTextureAccumulateConstantCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->RWTarget = GraphBuilder.CreateUAV(TargetTexture);
			PassParameters->TargetPosition = TargetPosition;
			PassParameters->ConstantValue = ConstantValue;
			PassParameters->Size = Size;
			PassParameters->Mask = bUseMask ? GraphBuilder.CreateSRV(Mask) : nullptr;
		}

		SHADER::FPermutationDomain ComputeShaderPermutationVector;
		ComputeShaderPermutationVector.Set<SHADER::FDimensionAccumulateByMask>(bUseMask);
		TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap, ComputeShaderPermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::AddAccumulateConstantRegionPass ([%.1f,%.1f,%.1f,%.1f] -> %s [%d,%d], size:%dx%d %s)",
				ConstantValue.R,
				ConstantValue.G,
				ConstantValue.B,
				ConstantValue.A,
				TargetTexture->Name,
				TargetPosition.X,
				TargetPosition.Y,
				Size.X,
				Size.Y,
				bUseMask ? TEXT("Masked") : TEXT("")),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Size, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	void AddCopyMirroredTexturePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		FIntPoint SourcePosition, FIntPoint TargetPosition, FIntPoint Size, bool bAlphaOnly)
	{
		const FIntPoint CopySize = Size == FIntPoint::ZeroValue ? TargetTexture->Desc.Extent : Size;

		typedef FCopyTexturePS SHADER;

		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->Source = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTexture));
			PassParameters->SourceOffset = SourcePosition;
			PassParameters->TextureSize = SourceTexture->Desc.Extent;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(TargetTexture, ERenderTargetLoadAction::ENoAction);
		}

		const FIntRect ViewRect(TargetPosition, TargetPosition + CopySize);

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FCopyTexturePS> PixelShader(ShaderMap);

		FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		if (bAlphaOnly)
		{
			BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_One, BO_Add, BF_One, BF_Zero>::GetRHI();
		}

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("CopyTexture (%s -> %s,Mirrored%s)",
				SourceTexture->Name,
				TargetTexture->Name,
				bAlphaOnly ? TEXT(" AlphaOnly") : TEXT("")),
			PixelShader,
			PassParameters,
			ViewRect,
			BlendState
		);
	}

	void AddCopyMirroredTexturePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture, int32 Channel, ETextureCopyType CopyType,
		FIntPoint SourcePosition, FIntPoint TargetPosition, FIntPoint Size)
	{
		const FIntPoint CopySize = Size == FIntPoint::ZeroValue ? TargetTexture->Desc.Extent : Size;

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		typedef FCopyTextureSingleChannelCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->CopySource = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTexture));
			PassParameters->RWCopyTarget = GraphBuilder.CreateUAV(TargetTexture);
			PassParameters->SourceOffset = SourcePosition;
			PassParameters->TargetOffset = TargetPosition;
			PassParameters->CopySize = CopySize;
			PassParameters->Channel = Channel;
			PassParameters->TextureSize = SourceTexture->Desc.Extent;
		}

		SHADER::FPermutationDomain PermutationDomainVector;
		{
			PermutationDomainVector.Set<SHADER::FDimTextureCopyType>(CopyType);
		}

		TShaderMapRef<SHADER> ComputeShader(ShaderMap, PermutationDomainVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CopyTextureCS (%s [%d,%d] -> %s [%d,%d], size:%dx%d, c=%d)",
				SourceTexture->Name,
				SourcePosition.X,
				SourcePosition.Y,
				TargetTexture->Name,
				TargetPosition.X,
				TargetPosition.Y,
				CopySize.X,
				CopySize.Y,
				Channel),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(CopySize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	void AddNormalizeRadianceVariancePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& Albedo, const FRDGTextureRef& RadianceVariance)
	{
		FIntPoint Size = RadianceVariance->Desc.Extent;
		typedef FNormalizeRadianceVarianceByAlbedoCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->Albedo = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Albedo));
			PassParameters->RWRadianceVariance = GraphBuilder.CreateUAV(RadianceVariance);
			PassParameters->Size = Size;
		}

		TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::AddNormalizeRadianceVariancePass (%s.RadianceVariance / %s, size:%dx%d)",
				RadianceVariance->Name,
				Albedo->Name,
				Size.X,
				Size.Y),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Size, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	//--------------------------------------------------------------------------------------------------------------------
	// Radiance normalization
	FRDGTextureRef GetPreAlbedoDivideMask(FRDGBuilder& GraphBuilder, const FSceneView& View, const FRDGTextureRef& Normal, const FRDGTextureRef& NormalVariance)
	{
		FRDGTextureDesc Desc = Normal->Desc;
		Desc.Format = PF_R8_UINT;
		FRDGTextureRef MaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("NFOR.MaskTexture"));
		{
			typedef FClassifyPreAlbedoDivideMaskIdCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->Normal = GraphBuilder.CreateSRV(Normal);
				PassParameters->NormalVariance = GraphBuilder.CreateSRV(NormalVariance);
				PassParameters->TextureSize = Desc.Extent;
				PassParameters->RWMask = GraphBuilder.CreateUAV(MaskTexture);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::ClassifyPreAlbedoDivideMaskIdCS (size:%dx%d)",
					Desc.Extent.X,
					Desc.Extent.Y),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(Desc.Extent, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}

		return MaskTexture;
	}

	void AddAdjustFeatureRangePass(FRDGBuilder& GraphBuilder, const FFeatureDesc& FeatureDesc, float MaxValue)
	{
		checkf(FeatureDesc.Data.NumOfChannel == 4, TEXT("Only feature with 4 channels can be adjusted"));
		checkf(FeatureDesc.VarianceType != EVarianceType::Colored, TEXT("Feature variance of type EVarianceType::Colored cannot be adjusted"));
		
		if (MaxValue <= 0)
		{
			return;
		}

		FIntPoint Size = FeatureDesc.Data.Image->Desc.Extent;
		typedef FAdjustFeatureRangeCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->RWImage = GraphBuilder.CreateUAV(FeatureDesc.Data.Image);
			PassParameters->RWImageVariance = GraphBuilder.CreateUAV(FeatureDesc.Variance.Image);
			PassParameters->Size = Size;
			PassParameters->VarianceChannelOffset = FeatureDesc.Variance.ChannelOffset;
			PassParameters->MaxValue = MaxValue;
		}

		SHADER::FPermutationDomain ComputeShaderPermutationVector;
		{
			ComputeShaderPermutationVector.Set<SHADER::FDimensionVarianceType>(FeatureDesc.VarianceType);
		}

		TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::AddAdjustFeatureRangePass (%s, MaxValue=%.2f, size:%dx%d)",
				FeatureDesc.Data.Image->Name,
				MaxValue,
				Size.X,
				Size.Y),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Size, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	//--------------------------------------------------------------------------------------------------------------------
	// Non-local mean weight and filtering
	FNonLocalMeanParameters GetNonLocalMeanParameters(int32 PatchSize, int32 PatchDistance, float Bandwidth)
	{
		FNonLocalMeanParameters NonLocalMeanParameters;
		NonLocalMeanParameters.PatchSize = PatchSize;
		NonLocalMeanParameters.PatchDistance = PatchDistance;
		NonLocalMeanParameters.Bandwidth = Bandwidth;

		return NonLocalMeanParameters;
	}

	FNonLocalMeanParameters GetFeatureNonLocalMeanParameters(float Bandwidth)
	{
		const int32 PatchSize = GetNonLocalMeanFeaturePatchSize();
		const int32 PatchDistance = GetNonLocalMeanFeaturePatchDistance();

		return GetNonLocalMeanParameters(PatchSize, PatchDistance, Bandwidth);
	}

	EPixelFormat GetWeightLayoutPixelFormat(ENonLocalMeanWeightLayout WeightLayout)
	{
		EPixelFormat DefaultFormat = PF_R32_FLOAT;
		if (WeightLayout == ENonLocalMeanWeightLayout::Float4xWxHxNumOfWeightsPerPixelByFloat4)
		{
			DefaultFormat = PF_A32B32G32R32F;
		}

		return DefaultFormat;
	}

	void ApplyNonLocalMeanFilter(
		FRDGBuilder& GraphBuilder,
		const FNonLocalMeanParameters& NonLocalMeanParameters,
		const FNFORTextureDesc& Texture,
		const FNFORTextureDesc& Variance,
		EVarianceType VarianceType,
		const FRDGTextureRef& FilteredTexture,
		const FNonLocalMeanWeightDesc& WeightBufferDesc)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		FIntPoint TextureSize = Texture.Image->Desc.Extent;
		const FRDGTextureRef VarianceTexture = Variance.Image ? Variance.Image : GSystemTextures.GetBlackDummy(GraphBuilder);
		bool bUseWeightBuffer = WeightBufferDesc.WeightBuffer && WeightBufferDesc.WeightLayout != ENonLocalMeanWeightLayout::None;
		ENonLocalMeanWeightLayout WeightLayout = bUseWeightBuffer ? WeightBufferDesc.WeightLayout : ENonLocalMeanWeightLayout::None;
		FIntRect FilteringRegion = bUseWeightBuffer ? WeightBufferDesc.Region : FIntRect(FIntPoint::ZeroValue, TextureSize);

		typedef FNonLocalMeanFilteringCS SHADER;

		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->NLMParams = NonLocalMeanParameters;
			PassParameters->Image = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Texture.Image));
			PassParameters->Variance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(VarianceTexture));
			PassParameters->TextureSize = TextureSize;
			PassParameters->VarianceChannelOffset = Variance.ChannelOffset;
			PassParameters->DenoisingChannelCount = Texture.ChannelCount;
			PassParameters->DenoisedImage = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(FilteredTexture));
			PassParameters->FilteringRegion = FilteringRegion;

			if (bUseWeightBuffer)
			{
				PassParameters->NonLocalMeanWeights = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(WeightBufferDesc.WeightBuffer, 
					GetWeightLayoutPixelFormat(WeightLayout)));
			}
		}

		SHADER::FPermutationDomain ComputeShaderPermutationVector;
		{
			ComputeShaderPermutationVector.Set<SHADER::FDimensionVarianceType>(VarianceType);
			ComputeShaderPermutationVector.Set<SHADER::FDimensionUseGuide>(false);
			ComputeShaderPermutationVector.Set<SHADER::FDimensionImageChannelCount>(Texture.NumOfChannel);
			ComputeShaderPermutationVector.Set<SHADER::FDimPreAlbedoDivide>(GetPreAlbedoDivideRecoverPhase());
			ComputeShaderPermutationVector.Set<SHADER::FDimWeightLayout>(WeightLayout);
		}

		TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap, ComputeShaderPermutationVector);

		FIntPoint FilteringSize = FilteringRegion.Size();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::FeatureFiltering(%s, %s Dim=%d,%d)", 
				Texture.Image->Name, 
				GetNonLocalMeanWeightLayoutName(WeightLayout),
				FilteringSize.X, 
				FilteringSize.Y),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FilteringSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
	}

	void GetNLMWeigthsWithAtlas(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FRadianceDesc& SourceRadiance,
		const FRadianceDesc& TargetRadiance,
		const FNonLocalMeanWeightDesc& NonLocalMeanWeightDesc,
		const FNonLocalMeanParameters& NonLocalMeanParameters)
	{
		const int32 SearchingPatchSize = (NonLocalMeanParameters.PatchDistance * 2 + 1);
		const int32 NumberOfWeightsPerPixel = SearchingPatchSize * SearchingPatchSize;
		const FIntPoint TextureSize = SourceRadiance.Data.Image->Desc.Extent;
		// If we expect float4xWxHx[NumOfWeight/4], we can directly write through.
		const bool bShouldUseLocalBuffer = NonLocalMeanWeightDesc.WeightLayout != ENonLocalMeanWeightLayout::Float4xWxHxNumOfWeightsPerPixelByFloat4;
		const bool bSeparateSourceTarget = (SourceRadiance.Data.Image != TargetRadiance.Data.Image)|| !bShouldUseLocalBuffer;

		const FIntRect SeparableFilteringRegion = NonLocalMeanWeightDesc.Region.Inner(-(NonLocalMeanParameters.PatchDistance + NonLocalMeanParameters.PatchSize));
		const FIntPoint SeparableFilteringExtent = SeparableFilteringRegion.Size();
		const FIntPoint WeightQueryRegionExtent = NonLocalMeanWeightDesc.Region.Inner(-(NonLocalMeanParameters.PatchDistance)).Size();
		
		// Estimate the number of dispatches required using the atlas
		const FIntPoint NLMWeightAtlasExtent = GetNonLocalMeanAtlasSize(SeparableFilteringExtent);
		const FIntVector DispatchTileVector = FIntVector(
			FMath::DivideAndRoundDown(NLMWeightAtlasExtent.X, SeparableFilteringExtent.X),
			FMath::DivideAndRoundDown(NLMWeightAtlasExtent.Y, SeparableFilteringExtent.Y), 1);

		const int32 HalfOffsetSearchCount = (NumberOfWeightsPerPixel / 2 + 1);
		const int32 DispatchTileCount = DispatchTileVector.X * DispatchTileVector.Y;

		const ENonLocalMeanAtlasType NonLocalMeanAtlasType = GetNonLocalMeanAtlasType();
		const int32 NumSymmetricPairsPerPixel = static_cast<int32>(NonLocalMeanAtlasType) + 1;
		const int32 SingleDispatchOffsetSearchCount = DispatchTileCount * NumSymmetricPairsPerPixel;
		const int32 NumOfDispatch = FMath::DivideAndRoundUp(HalfOffsetSearchCount, SingleDispatchOffsetSearchCount);

		// Allocate the atlas where each pixel stores two symmetric distance/weights, and the temporal buffer.
		FRDGTextureDesc NLMWeightAtlasDesc = SourceRadiance.Data.Image->Desc;
		int32 NonLocalMeanWeightsBytesPerElement = 0;
		{
			NLMWeightAtlasDesc.Extent = NLMWeightAtlasExtent;
			switch (NonLocalMeanAtlasType)
			{
			case ENonLocalMeanAtlasType::OneSymmetricPair:
				NLMWeightAtlasDesc.Format = PF_G32R32F;
				NonLocalMeanWeightsBytesPerElement = 2 * sizeof(float);
				break;
			case ENonLocalMeanAtlasType::TwoSymmetricPair:
			default:
				NonLocalMeanWeightsBytesPerElement = 4 * sizeof(float);
				NLMWeightAtlasDesc.Format = PF_A32B32G32R32F; 
				break;
			}
		}
		
		FRDGTextureRef NLMWeightAtlas[2] = {
			GraphBuilder.CreateTexture(NLMWeightAtlasDesc, TEXT("NFOR.NLMWeightAtlas0")),
			GraphBuilder.CreateTexture(NLMWeightAtlasDesc, TEXT("NFOR.NLMWeightAtlas1"))
		};
		
		const int32 TotalNumTilesToFill = FMath::DivideAndRoundUp(HalfOffsetSearchCount, NumSymmetricPairsPerPixel);
		FRDGBufferRef  NonLocalMeanWeights = nullptr;
		if (bShouldUseLocalBuffer)
		{
			size_t NonLocalMeanWeightsCount = NumSymmetricPairsPerPixel * (WeightQueryRegionExtent.X * WeightQueryRegionExtent.Y) * TotalNumTilesToFill;
			FRDGBufferDesc NonLocalMeanWeightsDesc = FRDGBufferDesc::CreateBufferDesc(NonLocalMeanWeightsBytesPerElement, NonLocalMeanWeightsCount);
			NonLocalMeanWeights = GraphBuilder.CreateBuffer(NonLocalMeanWeightsDesc, TEXT("NFOR.NonLocalMeanWeights"));
		}
		else
		{
			NonLocalMeanWeights = NonLocalMeanWeightDesc.WeightBuffer;
		}
		// Summery
		// 1. For each dispatch:
		//		Calcualte offset for each tile (SeparableFilteringRegion) in the atlas
		//		Horizontal filter to second atlas
		//		Vertical filter to buffer
		// 2. Reshape the buffer for later use


		struct FSeperableFilterPassInfo
		{
			const TCHAR* PassName;
			FRDGTextureRef Input;
			FRDGTextureRef Output;
			FIntPoint	GroupCountXY;
			FNonLocalMeanSeperableFilterPatchSqauredDistanceCS::ESeperablePassType SeperablePassType;
		};

		const int NumOfSeperablePass = 2;

		//	Horizontal requires all rows
		//	Vertical filtering only requires the weight query regions and stores to a buffer.
		const FSeperableFilterPassInfo SeperableFilterPassInfo[NumOfSeperablePass] =
		{
			{	TEXT("NFOR::SeperableHorizontal"),	NLMWeightAtlas[0], NLMWeightAtlas[1], FIntPoint(SeparableFilteringExtent.X, SeparableFilteringExtent.Y),
			FNonLocalMeanSeperableFilterPatchSqauredDistanceCS::ESeperablePassType::Horizontal},			//Separable horizontal
			{	TEXT("NFOR::SeperableVertical"),	NLMWeightAtlas[1],			 nullptr, FIntPoint(WeightQueryRegionExtent.X, WeightQueryRegionExtent.Y),
			FNonLocalMeanSeperableFilterPatchSqauredDistanceCS::ESeperablePassType::Vertical},				//Separable Vertical
		};

		for (int32 DispatchId = 0; DispatchId < NumOfDispatch; ++DispatchId)
		{
			const int32 NumSymmetricPairsPerDispatch = SingleDispatchOffsetSearchCount - FMath::Max((DispatchId + 1) * SingleDispatchOffsetSearchCount - HalfOffsetSearchCount, 0);
			const int32 NumOfTilesToFillThisDispatch = FMath::DivideAndRoundUp(NumSymmetricPairsPerDispatch, NumSymmetricPairsPerPixel);
			const FIntVector DispatchRegionSize = FIntVector(SeparableFilteringExtent.X, SeparableFilteringExtent.Y, NumOfTilesToFillThisDispatch);

			FNonLocalMeanWeightAtlasDispatchParameters NLMWeightAtlasDispatchParameters;
			{
				NLMWeightAtlasDispatchParameters.DispatchId = DispatchId;
				NLMWeightAtlasDispatchParameters.DispatchTileSize = FIntPoint(DispatchTileVector.X, DispatchTileVector.Y);
				NLMWeightAtlasDispatchParameters.DispatchTileCount = DispatchTileCount;
				NLMWeightAtlasDispatchParameters.SeparableFilteringRegion = SeparableFilteringRegion;
				NLMWeightAtlasDispatchParameters.DispatchRegionSize = DispatchRegionSize;
			}

			// Get squared distance, each tile pixel holds NumSymmetricPairsPerPixel symmetric pairs.
			{
				typedef FNonLocalMeanGetSqauredDistanceToAtlasCS SHADER;

				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->CommonParameters.NLMParams = NonLocalMeanParameters;
					PassParameters->NLMWeightAtlasDispatchParameters = NLMWeightAtlasDispatchParameters;
					PassParameters->CommonParameters.Image = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceRadiance.Data.Image));
					PassParameters->CommonParameters.Variance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceRadiance.Variance.Image));
					PassParameters->CommonParameters.TextureSize = TextureSize;
					PassParameters->CommonParameters.VarianceChannelOffset = SourceRadiance.Variance.ChannelOffset;

					if (bSeparateSourceTarget)
					{
						PassParameters->TargetImage = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(TargetRadiance.Data.Image));
						PassParameters->TargetVariance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(TargetRadiance.Variance.Image));
					}
					PassParameters->RWNLMWeightAtlas = GraphBuilder.CreateUAV(NLMWeightAtlas[0]);
					PassParameters->NLMWeightAtlasSize = NLMWeightAtlasExtent;
				}

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				{
					ComputeShaderPermutationVector.Set<SHADER::FDimensionVarianceType>(SourceRadiance.VarianceType);
					ComputeShaderPermutationVector.Set<SHADER::FDimensionImageChannelCount>(SourceRadiance.Data.NumOfChannel);
					ComputeShaderPermutationVector.Set<SHADER::FDimensionSeparateSourceTarget>(bSeparateSourceTarget);
					ComputeShaderPermutationVector.Set<SHADER::FDimAtlasType>(NonLocalMeanAtlasType);
					ComputeShaderPermutationVector.Set<SHADER::FDimPreAlbedoDivide>(GetPreAlbedoDivideRecoverPhase());
				}

				TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);
				
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NFOR::NonLocalMeanGetSqauredDistanceToAtlasCS (Rect=(%d,%d,%d,%d), pd=%d, DisaptchId,DispatchTileCount=%d,%d, GC=(%d,%d,%d))",
						SeparableFilteringRegion.Min.X,
						SeparableFilteringRegion.Min.Y,
						SeparableFilteringRegion.Max.X,
						SeparableFilteringRegion.Max.Y,
						NonLocalMeanParameters.PatchDistance,
						DispatchId,
						DispatchTileCount,
						DispatchRegionSize.X,
						DispatchRegionSize.Y,
						DispatchRegionSize.Z),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(DispatchRegionSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
			}

			// Horizontal and vertical filtering
			for (int SeperableFilteringId = 0; SeperableFilteringId < NumOfSeperablePass; ++SeperableFilteringId)
			{
				const FSeperableFilterPassInfo& PassInfo = SeperableFilterPassInfo[SeperableFilteringId];
				FIntVector SeperableRegionSize = FIntVector(PassInfo.GroupCountXY.X, PassInfo.GroupCountXY.Y, NumOfTilesToFillThisDispatch);

				typedef FNonLocalMeanSeperableFilterPatchSqauredDistanceCS SHADER;

				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->NLMParams = NonLocalMeanParameters;
					PassParameters->NLMWeightAtlasDispatchParameters = NLMWeightAtlasDispatchParameters;
					PassParameters->NLMWeightAtlasSource = GraphBuilder.CreateSRV(PassInfo.Input);

					if (PassInfo.SeperablePassType == SHADER::ESeperablePassType::Horizontal)
					{
						PassParameters->RWNLMWeightAtlasTarget = GraphBuilder.CreateUAV(PassInfo.Output);
					}
					else
					{
						// Vertical pass directly write to the weight buffer
						PassParameters->RWNLMWeights = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(NonLocalMeanWeights, NLMWeightAtlasDesc.Format));
					}
					PassParameters->SeperableRegionSize = SeperableRegionSize;
				}

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				{
					ComputeShaderPermutationVector.Set<SHADER::FDimensionSeperablePassType>(PassInfo.SeperablePassType);
					ComputeShaderPermutationVector.Set<SHADER::FDimAtlasType>(NonLocalMeanAtlasType);
					ComputeShaderPermutationVector.Set<SHADER::FDimPreAlbedoDivide>(GetPreAlbedoDivideRecoverPhase());
					ComputeShaderPermutationVector.Set<SHADER::FDimBufferPassThrough>(!bShouldUseLocalBuffer);
				}

				TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("%s ps=%d)",
						PassInfo.PassName,
						NonLocalMeanParameters.PatchSize),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(SeperableRegionSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
			}// End filtering
		} // End dispatch

		// Reshape the buffer from X*Y*Wb to W*X*Y and scatter the results
		if (bShouldUseLocalBuffer)
		{
			typedef FNonLocalMeanReshapeBufferCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->NLMParams = NonLocalMeanParameters;
				PassParameters->SourceBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(NonLocalMeanWeights, PF_G32R32F));
				PassParameters->RWTargetBuffer = GraphBuilder.CreateUAV(NonLocalMeanWeightDesc.WeightBuffer, PF_R32_FLOAT);
				PassParameters->SourceBufferDim = FIntVector4(NumSymmetricPairsPerPixel, WeightQueryRegionExtent.X, WeightQueryRegionExtent.Y, TotalNumTilesToFill);
				PassParameters->TargetBufferDim = FIntVector(NumberOfWeightsPerPixel, NonLocalMeanWeightDesc.Region.Size().X, NonLocalMeanWeightDesc.Region.Size().Y);
				PassParameters->HalfOffsetSearchCount = HalfOffsetSearchCount;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				ComputeShaderPermutationVector.Set<SHADER::FDimensionSeparateSourceTarget>(bSeparateSourceTarget);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionTargetWeightLayout>(NonLocalMeanWeightDesc.WeightLayout);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::NonLocalMeanReshapeBufferCS (Size=(%d,%d), HalfOffsetSearchCount=%d)",
					WeightQueryRegionExtent.X,
					WeightQueryRegionExtent.Y,
					HalfOffsetSearchCount),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(FIntVector(WeightQueryRegionExtent.X, WeightQueryRegionExtent.Y, HalfOffsetSearchCount), NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}
	}

	void ApplyNonLocalMeanFilterIfRequired(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FNonLocalMeanParameters& NonLocalMeanParameters,
		const FNFORTextureDesc& Texture,
		const FNFORTextureDesc& Variance,
		EVarianceType VarianceType,
		const FRDGTextureRef& FilteredTexture,
		int32 TileDownScale = 1)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "NonLocalMeanFiltering (%s)", Texture.Image->Name);

		if (ShouldApplyFeatureFiltering())
		{
			const FIntPoint TextureSize = Texture.Image->Desc.Extent;

			ENonLocalMeanWeightLayout WeightLayout = GetNonLocalMeanWeightLayout();
			FNonLocalMeanWeightDesc NonLocalMeanWeightDesc;
			NonLocalMeanWeightDesc.WeightLayout = WeightLayout;

			const bool bShouldApplyDirectFiltering = 
				(WeightLayout == ENonLocalMeanWeightLayout::None)
				|| !ShouldNonLocalMeanUseAtlas();

			if (bShouldApplyDirectFiltering)
			{
				NonLocalMeanWeightDesc.Region = FIntRect(FIntPoint(0), TextureSize);

				ApplyNonLocalMeanFilter(
					GraphBuilder,
					NonLocalMeanParameters,
					Texture,
					Variance,
					VarianceType,
					FilteredTexture,
					NonLocalMeanWeightDesc);
			}
			else
			{
				/**
				For each tile:
					Query the weights.
					Filter region with the weights.
				*/
				// TODO: refactor the tiling common code.

				const int SearchingPatchSize = (NonLocalMeanParameters.PatchDistance * 2 + 1);
				const int NumberOfWeightsPerPixel = SearchingPatchSize * SearchingPatchSize;

				const int32 NumOfTilesOneSide = FMath::Max(GetNumOfTiles(TextureSize) / TileDownScale, 1);
				FIntPoint NumOfTiles = FIntPoint(NumOfTilesOneSide, NumOfTilesOneSide);
				const int32 TotalTileCount = NumOfTilesOneSide * NumOfTilesOneSide;

				FIntPoint TileSize = FMath::DivideAndRoundUp(TextureSize, NumOfTiles);

				const int32 NonLocalMeanSingleFrameWeightSize = GetNonLocalMeanSingleFrameWeightBufferSize(TileSize, NumberOfWeightsPerPixel);
				const int32 BytesPerElement = sizeof(float);
				FRDGBufferDesc NonLocalMeanSingleFrameWeightsBufferDesc =
					FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NonLocalMeanSingleFrameWeightSize);
				FRDGBufferRef NonLocalMeanSingleFrameWeightsBuffer = GraphBuilder.CreateBuffer(NonLocalMeanSingleFrameWeightsBufferDesc, TEXT("NFOR.NLMFeatureFilteringWeightsBuffer"));

				FRadianceDesc ImageDesc(Texture, Variance, VarianceType);
				NonLocalMeanWeightDesc.WeightBuffer = NonLocalMeanSingleFrameWeightsBuffer;

				for (int i = 0; i < TotalTileCount; ++i)
				{
					int32 TileIndex = i;
					FIntPoint TileStartPoint = FIntPoint(TileIndex % NumOfTiles.X, TileIndex / NumOfTiles.X) * TileSize;
					FIntRect TileRegion = FIntRect(FIntPoint(0), TileSize) + TileStartPoint;

					NonLocalMeanWeightDesc.Region = TileRegion;

					GetNLMWeigthsWithAtlas(
						GraphBuilder,
						View,
						ImageDesc,
						ImageDesc,
						NonLocalMeanWeightDesc,
						NonLocalMeanParameters);

					ApplyNonLocalMeanFilter(
						GraphBuilder,
						NonLocalMeanParameters,
						Texture,
						Variance,
						VarianceType,
						FilteredTexture,
						NonLocalMeanWeightDesc);
				}
			}
		}
		else
		{
			AddCopyTexturePass(GraphBuilder, Texture.Image, FilteredTexture);
		}
	}

	void GetNLMWeights(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FRadianceDesc& SourceRadiance,
		const FRadianceDesc& TargetRadiance,
		const FRDGBufferRef& NonLocalMeanWeightsBuffer,
		FIntRect Region,
		const FNonLocalMeanParameters& NonLocalMeanParameters)
	{
		const int32 SearchingPatchSize = (NonLocalMeanParameters.PatchDistance * 2 + 1);
		const int32 NumberOfWeightsPerPixel = SearchingPatchSize * SearchingPatchSize;
		const FIntPoint TextureSize = SourceRadiance.Data.Image->Desc.Extent;
		const bool bSeparateSourceTarget = (SourceRadiance.Data.Image != TargetRadiance.Data.Image);		
		const bool bShouldNonLocalMeanUseAtlas = ShouldNonLocalMeanUseAtlas();

		RDG_EVENT_SCOPE(GraphBuilder, "NonLocalMeanGetWeights");

		FNonLocalMeanWeightDesc NonLocalMeanWeightDesc;
		NonLocalMeanWeightDesc.Region = Region;
		NonLocalMeanWeightDesc.WeightBuffer = NonLocalMeanWeightsBuffer;
		NonLocalMeanWeightDesc.WeightLayout = GetNonLocalMeanWeightLayout();

		// Query the non-local mean weights for the radiance.
		if (bShouldNonLocalMeanUseAtlas)
		{
			GetNLMWeigthsWithAtlas(GraphBuilder, View, SourceRadiance, TargetRadiance, NonLocalMeanWeightDesc, NonLocalMeanParameters);
		}
		else
		{
			typedef FNonLocalMeanWeightsCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->CommonParameters.NLMParams = NonLocalMeanParameters;
				PassParameters->CommonParameters.Image = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceRadiance.Data.Image));
				PassParameters->CommonParameters.Variance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceRadiance.Variance.Image));
				PassParameters->CommonParameters.TextureSize = TextureSize;
				PassParameters->CommonParameters.VarianceChannelOffset = SourceRadiance.Variance.ChannelOffset;

				PassParameters->RWNonLocalMeanWeights = GraphBuilder.CreateUAV(NonLocalMeanWeightsBuffer, PF_R32_FLOAT);
				PassParameters->Region = Region;

				if (bSeparateSourceTarget)
				{
					PassParameters->TargetImage = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(TargetRadiance.Data.Image));
					PassParameters->TargetVariance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(TargetRadiance.Variance.Image));
				}
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				ComputeShaderPermutationVector.Set<SHADER::FDimensionVarianceType>(EVarianceType::GreyScale);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionUseGuide>(false);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionImageChannelCount>(SourceRadiance.Data.NumOfChannel);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionSeparateSourceTarget>(bSeparateSourceTarget);
				ComputeShaderPermutationVector.Set<SHADER::FDimPreAlbedoDivide>(GetPreAlbedoDivideRecoverPhase());
				ComputeShaderPermutationVector.Set<SHADER::FDimTargetWeightLayout>(NonLocalMeanWeightDesc.WeightLayout);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::NonLocalMeanWeights (Rect=(%d,%d,%d,%d),ps=%d,pd=%d,bw=%.2f)",
					Region.Min.X,
					Region.Min.Y,
					Region.Max.X,
					Region.Max.Y,
					NonLocalMeanParameters.PatchSize,
					NonLocalMeanParameters.PatchDistance,
					NonLocalMeanParameters.Bandwidth),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(Region.Size(), NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}
	}

	//--------------------------------------------------------------------------------------------------------------------
	// Collaborative filtering
	//	1. Tiling
	void AddCopyTextureToBufferPass(
		FRDGBuilder& GraphBuilder,
		const FRDGTextureRef Source,
		const FRDGBufferRef Dest,
		int32 CopyChannelOffset,
		int32 CopyChannelCount,
		int32 NumberOfSourceChannel,
		int32 BufferChannelOffset,
		int32 BufferChannelSize,
		FIntRect CopyRegion)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		const int SourceChannelCount = NumberOfSourceChannel;
		FIntPoint TextureSize = Source->Desc.Extent;

		{
			typedef FCopyTextureToBufferCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->Source = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Source));
				PassParameters->Dest = GraphBuilder.CreateUAV(Dest, SHADER::GetDestFloatFormat(Dest->Desc.BytesPerElement));
				PassParameters->TextureSize = TextureSize;
				PassParameters->CopyChannelOffset = CopyChannelOffset;
				PassParameters->CopyChannelCount = CopyChannelCount;
				PassParameters->BufferChannelOffset = BufferChannelOffset;
				PassParameters->BufferChannelSize = BufferChannelSize;
				PassParameters->CopyRegion = CopyRegion;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			ComputeShaderPermutationVector.Set<SHADER::FDimensionSourceChannelCount>(SourceChannelCount);
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap, ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::CopyTextureToBuffer (Dim=%dx%d,s:%d:%d -> b:%d)",
					TextureSize.X,
					TextureSize.Y,
					CopyChannelOffset,
					CopyChannelOffset + CopyChannelCount - 1,
					BufferChannelOffset),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(CopyRegion.Size(), NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}
	}

	void AddCopyTextureToBufferPass(
		FRDGBuilder& GraphBuilder,
		const FRDGTextureRef Source,
		const FRDGBufferRef Dest,
		int32 CopyChannelOffset,
		int32 CopyChannelCount,
		int32 NumberOfSourceChannel,
		int32 BufferChannelOffset,
		int32 BufferChannelSize)
	{
		AddCopyTextureToBufferPass(
			GraphBuilder,
			Source,
			Dest,
			CopyChannelOffset,
			CopyChannelCount,
			NumberOfSourceChannel,
			BufferChannelOffset,
			BufferChannelSize,
			FIntRect(FIntPoint(0, 0), Source->Desc.Extent));
	}

	void AddNormalizeTexturePass( FRDGBuilder& GraphBuilder, const FRDGTextureRef& InputTexture)
	{
		FIntPoint TextureSize = InputTexture->Desc.Extent;
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		{
			typedef FNormalizeTextureCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->RWSource = GraphBuilder.CreateUAV(InputTexture);
				PassParameters->TextureSize = TextureSize;
			}

			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::AddNormalizeTexturePass (%dx%d)",
					TextureSize.X,
					TextureSize.Y),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(TextureSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}
	}


	//	2. Weighted Least-square solver
	FRDGBufferRef RegressionKernel::AllocateMatrixfBuffer(FRDGBuilder& GraphBuilder, int32 NumOfMatrices, int32 Dim0, int32 Dim1, const TCHAR* Name)
	{
		const int32 BytesPerElement = sizeof(float);

		FRDGBufferDesc BufferDesc =
			FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NumOfMatrices * Dim0 * Dim1);

		return GraphBuilder.CreateBuffer(BufferDesc, Name);
	}

	FRDGBufferRef RegressionKernel::FInPlaceBatchedMatrixMultiplicationCS::AllocateResultBuffer( FRDGBuilder& GraphBuilder, FIntPoint Size, int32 F, int32 A)
	{
		return AllocateMatrixfBuffer(GraphBuilder, Size.X * Size.Y, F, A, TEXT("NFOR.Matrix.Result"));
	}

	FRDGBufferRef ApplyBatchedInPlaceMatrixMultiplication(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		FRDGBufferRef X,
		FIntPoint XDim,
		FRDGBufferRef W,
		int32 WDim,
		FIntPoint TextureSize,
		int PatchDistance,
		RegressionKernel::EWeightedMultiplicationType MultiplicationType = RegressionKernel::EWeightedMultiplicationType::Quadratic,
		FRDGBufferRef Y = nullptr,
		FIntPoint YDim = 0)
	{
		FRDGBufferRef ResultMatrix = nullptr;
		{
			typedef RegressionKernel::FInPlaceBatchedMatrixMultiplicationCS SHADER;
			const bool GeneralizedMultiplication = MultiplicationType == RegressionKernel::EWeightedMultiplicationType::Generalized;
			const bool bFeatureAddConstant = ShouldFeatureAddConstant();
			const int BufferXDimWithConstant = XDim.Y + (bFeatureAddConstant ? 1 : 0);
			FIntPoint ResultMatrixDimension = FIntPoint(BufferXDimWithConstant, GeneralizedMultiplication ? YDim.Y : BufferXDimWithConstant);
			int32 SamplingStep = GetSamplingStep(WDim, ResultMatrixDimension.X * ResultMatrixDimension.Y);
			ENonLocalMeanWeightLayout NonLocalMeanWeightLayout = GetNonLocalMeanWeightLayout();

			ResultMatrix = SHADER::AllocateResultBuffer(GraphBuilder, TextureSize, ResultMatrixDimension.X, ResultMatrixDimension.Y);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->X = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(X, SHADER::GetXYFloatFormat(X->Desc.BytesPerElement)));
				PassParameters->XDim = XDim;

				PassParameters->W = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(W, GetWeightLayoutPixelFormat(NonLocalMeanWeightLayout)));
				PassParameters->WDim = WDim;


				if (GeneralizedMultiplication)
				{
					PassParameters->Y = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Y, SHADER::GetXYFloatFormat(Y->Desc.BytesPerElement)));
					PassParameters->YDim = YDim;
				}
				else
				{
					PassParameters->Y = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(X, SHADER::GetXYFloatFormat(X->Desc.BytesPerElement)));
					PassParameters->YDim = XDim;
				}

				PassParameters->TextureSize = TextureSize;
				PassParameters->PatchDistance = PatchDistance;
				PassParameters->NumOfWeigthsPerPixelPerFrame = (PatchDistance * 2 + 1) * (PatchDistance * 2 + 1);
				PassParameters->NumOfTemporalFrames = WDim / PassParameters->NumOfWeigthsPerPixelPerFrame;
				PassParameters->SourceFrameIndex = GetDenoisingFrameIndex(View, PassParameters->NumOfTemporalFrames);
				PassParameters->SamplingStep = SamplingStep;
				PassParameters->Result = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(ResultMatrix, PF_R32_FLOAT));
			}

			//TODO: clean up mutation.
			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				ComputeShaderPermutationVector.Set<SHADER::FDimWeightedMultiplicationType>(MultiplicationType);
				ComputeShaderPermutationVector.Set<SHADER::FDimAddConstantFeatureDim>(bFeatureAddConstant);
				ComputeShaderPermutationVector.Set<SHADER::FDimOptimizeTargetMatrixMultiplication>(true);
				ComputeShaderPermutationVector.Set<SHADER::FDimNumFeature>(BufferXDimWithConstant);
				ComputeShaderPermutationVector.Set<SHADER::FDimUseSamplingStep>(SamplingStep > 1);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionWeightLayout>(NonLocalMeanWeightLayout);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::Matrix Multiplication (%s%s)",
					RegressionKernel::GetEventName(MultiplicationType),
					bFeatureAddConstant? TEXT(" +Const. Feature":TEXT(""))),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(TextureSize, SHADER::GetThreadGroupSize()));
		}

		return ResultMatrix;
	}

	void ReconstructByFrame(
		FRDGBuilder& GraphBuilder,
		FRDGBufferRef Feature,
		FRDGBufferRef ReconstructionWeights,
		FRDGBufferRef NonLocalMeanWeightsBuffer,
		FRDGTextureRef FilteredRadiance,
		FRDGTextureRef SourceAlbedo,
		FWeightedLSRDesc WeightedLSRDesc,
		int FrameIndex,
		RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType ReconstructionType = RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType::Scatter
	)
	{

		checkf(FrameIndex < WeightedLSRDesc.NumOfFrames,
			TEXT("FrameIndex should be less than total number of frames: %d < %d failed:"), FrameIndex, WeightedLSRDesc.NumOfFrames);
		
		FRDGTextureRef ReconstructionBuffer64 = nullptr;
		FRDGBufferRef ReconstructionBuffer = nullptr;

		FIntPoint TextureSize = FilteredRadiance->Desc.Extent;
		ENonLocalMeanWeightLayout NonLocalMeanWeightLayout = GetNonLocalMeanWeightLayout();

		{
			typedef RegressionKernel::FReconstructSpatialTemporalImage SHADER;
			if (ReconstructionType == SHADER::EReconstructionType::Scatter)
			{
				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f),TextureSize.X * TextureSize.Y);
				ReconstructionBuffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("NFOR.WeightedLSR.ReconstructionBuffer"));

				const EPixelFormat PixelFormat64 = GPixelFormats[PF_R64_UINT].Supported ? PF_R64_UINT : PF_R32G32_UINT;

				const FRDGTextureDesc ReconstructionBufferDesc = FRDGTextureDesc::Create2D(
					FIntPoint(TextureSize.X*4, TextureSize.Y),
					PixelFormat64,
					FClearValueBinding::None,
					TexCreate_RenderTargetable|TexCreate_ShaderResource | TexCreate_UAV | ETextureCreateFlags::Atomic64Compatible);

				ReconstructionBuffer64 = GraphBuilder.CreateTexture(ReconstructionBufferDesc, TEXT("NFOR.WeightedLSR.ReconstructionBuffer64"));

				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ReconstructionBuffer), 0);
				AddClearRenderTargetPass(GraphBuilder, ReconstructionBuffer64, FLinearColor::Transparent);
			}

			const bool bFeatureAddConstant = ShouldFeatureAddConstant();
			const int32 NumOfAdditionalFeatures = (bFeatureAddConstant ? 1 : 0);

			FIntPoint XDimension = FIntPoint(WeightedLSRDesc.NumOfWeightsPerPixel, WeightedLSRDesc.NumOfFeatureChannelsPerFrame);
			const int PatchDistance = (FMath::Sqrt(float(WeightedLSRDesc.NumOfWeightsPerPixel / WeightedLSRDesc.NumOfFrames)) - 1) / 2;
			const int TotalNumOfFeaturesPerFrame = WeightedLSRDesc.NumOfFeatureChannelsPerFrame + NumOfAdditionalFeatures;
			FIntPoint BDim = FIntPoint(TotalNumOfFeaturesPerFrame, WeightedLSRDesc.NumOfRadianceChannelsPerFrame);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->X = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Feature, SHADER::GetXFloatFormat(Feature->Desc.BytesPerElement)));
				PassParameters->W = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(NonLocalMeanWeightsBuffer, GetWeightLayoutPixelFormat(NonLocalMeanWeightLayout)));
				PassParameters->B = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(ReconstructionWeights, PF_R32_FLOAT));
				PassParameters->RWReconstruction = GraphBuilder.CreateUAV(FilteredRadiance);
				PassParameters->RWReconstructBuffer = ReconstructionBuffer ? GraphBuilder.CreateUAV(ReconstructionBuffer) : nullptr;
				PassParameters->RWReconstructBuffer64 = ReconstructionBuffer ? GraphBuilder.CreateUAV(ReconstructionBuffer64) : nullptr;

				PassParameters->XDim = XDimension;
				PassParameters->WDim = WeightedLSRDesc.NumOfWeightsPerPixel;
				PassParameters->BDim = BDim;

				PassParameters->TextureSize = FIntPoint(WeightedLSRDesc.Width, WeightedLSRDesc.Height);
				PassParameters->PatchDistance = PatchDistance;
				PassParameters->FrameIndex = FrameIndex;

				PassParameters->NumOfTemporalFrames = WeightedLSRDesc.NumOfFrames;
				PassParameters->NumOfWeigthsPerPixelPerFrame = WeightedLSRDesc.NumOfWeightsPerPixel / WeightedLSRDesc.NumOfFrames;
			}
			
			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				ComputeShaderPermutationVector.Set<SHADER::FDimReconstructionType>(ReconstructionType);
				ComputeShaderPermutationVector.Set<SHADER::FDimPreAlbedoDivide>(GetPreAlbedoDivideRecoverPhase());
				ComputeShaderPermutationVector.Set<SHADER::FDimNumFeature>(BDim.X);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionWeightLayout>(NonLocalMeanWeightLayout);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::Reconstruction(T=%d,%s)",
					FrameIndex,
					SHADER::GetEventName(ReconstructionType)),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(FIntPoint(WeightedLSRDesc.Width, WeightedLSRDesc.Height), NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}

		if (ReconstructionBuffer)
		{
			typedef FAccumulateBufferToTextureCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->StructuredBufferSource = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(ReconstructionBuffer));
				PassParameters->ReconstructBuffer64 = GraphBuilder.CreateSRV(ReconstructionBuffer64);
				PassParameters->RWTarget = GraphBuilder.CreateUAV(FilteredRadiance);
				PassParameters->TextureSize = TextureSize;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				ComputeShaderPermutationVector.Set<SHADER::FDimPreAlbedoDivide>(GetPreAlbedoDivideRecoverPhase());
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::AccumulateBufferToTexture(%dx%d)", TextureSize.X, TextureSize.Y),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(TextureSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}
	}

	void ApplyLinearSolverGPU(
		FRDGBuilder& GraphBuilder,
		FRDGBufferRef AMatrix,
		FRDGBufferRef BMatrix,
		FIntPoint BDim,
		FRDGBufferRef ReconstructionWeights,
		const FWeightedLSRDesc& WeightedLSRDesc,
		int32 TotalNumOfFeaturesPerFrame,
		int32 NumOfElements,
		int32 NumOfElementsPerRow
	)
	{

		RDG_EVENT_SCOPE(GraphBuilder, "BatchedLinearSolver");

		// Summary of approximate ground truth solver
		// 1. Apply Cholesky decomposition with lambda = 0. and output failed indices.
		// 2. Apply Cholesky decomposition with lambda = 1e-6 on failed, and output both failed and succeeded indices.
		// 3. For failed indices, fallback to newton iterative method.
		// 4. For succeeded indices, iteratively refine with better lambda.
		// Summary of NewtonCholesky
		// 1. Apply Cholesky decomposition with lambda = 1e-3, fine tune with 3 iterations of Newton. If any inversion failed, output the failed indices.
		// 2. For failed indices, apply the standard newton iteration method. 
		// Other wise, solve based on the SolverType in a single pass.

		RegressionKernel::FLinearSolverCS::ESolverType SolverType = GetLinearSolverType();
		const bool bApproximateGroundTruthSolver = SolverType == RegressionKernel::FLinearSolverCS::ESolverType::MAX;
		const bool bUseSuccessAndFailIndexBuffer = bApproximateGroundTruthSolver || SolverType == RegressionKernel::FLinearSolverCS::ESolverType::NewtonCholesky;
		// Success count | Failed count | sidx... <--->     fidx|
		// One for read, one for write
		FRDGBufferRef SuccessAndFailIndexBuffer[2] = { nullptr, nullptr};
		if (bUseSuccessAndFailIndexBuffer)
		{
			const int32 BytesPerElement = sizeof(uint32);
			FRDGBufferDesc SuccessAndFailIndexBufferDesc = FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NumOfElements + 2);
			SuccessAndFailIndexBuffer[0] = GraphBuilder.CreateBuffer(SuccessAndFailIndexBufferDesc, TEXT("NFOR.LinearSolver.SuccessAndFailIndexBuffer0"));
			
			// Initialize the first two elements to 0 for SuccessAndFailIndexBuffer.
			FRDGBufferDesc IndicesHeadBufferDesc =
				FRDGBufferDesc::CreateBufferDesc(BytesPerElement, 2);
			FRDGBufferRef IndicesHeadBuffer = GraphBuilder.CreateBuffer(IndicesHeadBufferDesc, TEXT("NFOR.LinearSolver.IndexHeadBuffer"));

			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(IndicesHeadBuffer, PF_R32_UINT), 0);
			AddCopyBufferPass(GraphBuilder, SuccessAndFailIndexBuffer[0], 0, IndicesHeadBuffer, 0, BytesPerElement * 2);
			
			if (bApproximateGroundTruthSolver)
			{
				SuccessAndFailIndexBuffer[1] = GraphBuilder.CreateBuffer(SuccessAndFailIndexBufferDesc, TEXT("NFOR.LinearSolver.SuccessAndFailIndexBuffer1"));
				AddCopyBufferPass(GraphBuilder, SuccessAndFailIndexBuffer[1], 0, IndicesHeadBuffer, 0, BytesPerElement * 2);
			}
		}

		RegressionKernel::FLinearSolverCS::FParameters CommonPassParameters;
		{
			CommonPassParameters.A = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(AMatrix, PF_R32_FLOAT));
			CommonPassParameters.ADim = FIntPoint(TotalNumOfFeaturesPerFrame, TotalNumOfFeaturesPerFrame);
			CommonPassParameters.B = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(BMatrix, PF_R32_FLOAT));
			CommonPassParameters.BDim = BDim;
			CommonPassParameters.Result = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(ReconstructionWeights, PF_R32_FLOAT));
			CommonPassParameters.NumOfElements = NumOfElements;
			CommonPassParameters.NumOfElementsPerRow = NumOfElementsPerRow;
			CommonPassParameters.Lambda = 0.0f;
			CommonPassParameters.MinLambda = 0.0f;
		}

		// First multi-pass or the single pass based on SolverType.
		{
			typedef RegressionKernel::FLinearSolverCS SHADER;
			SHADER::ESolverType FirstPassSolverType = bApproximateGroundTruthSolver ? SHADER::ESolverType::Cholesky : SolverType;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				*PassParameters = CommonPassParameters;
				//Magnitude of X^TWX element value increases with the number of frames, and the number of elements selected to 
				// estimate the weights. GetLinearSolverCholeskyLambda() returns the lambda for a single frame.
				PassParameters->Lambda = GetLinearSolverCholeskyLambda();
				PassParameters->MinLambda = 1e-3 * WeightedLSRDesc.NumOfFrames; // Experimental value.
				if (bApproximateGroundTruthSolver)
				{
					PassParameters->RWSuccessAndFailIndexBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(SuccessAndFailIndexBuffer[0], PF_R32_UINT));
					PassParameters->Lambda = 0.0f;
					PassParameters->MinLambda = 0.0f;
				}
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				checkf(BDim.X >= 6 && BDim.X <= 8, TEXT("Number of features should be between 6 and 8"));
				ComputeShaderPermutationVector.Set<SHADER::FDimNumFeature>(BDim.X);
				ComputeShaderPermutationVector.Set<SHADER::FDimSolverType>(FirstPassSolverType);
				ComputeShaderPermutationVector.Set<SHADER::FDimOutputIndices>(bApproximateGroundTruthSolver);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::BatchedLinearSolver(F=%d, C=%d, %s)", 
					BDim.X, BDim.Y, 
					GetLinearSolverTypeName(SolverType)),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(FIntPoint(WeightedLSRDesc.Width, WeightedLSRDesc.Height), NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
		}

		// Early out for Newton or Cholesky method.
		if (!bUseSuccessAndFailIndexBuffer)
		{
			return;
		}

		FRDGBufferRef IndirectDispatchArgsBuffer = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("NFOR.LinearSolver.IndirectDispatchBuffer"));

		{
			{
				// Build the indirect dispatch parameters on failed
				typedef RegressionKernel::FLinearSolverBuildIndirectDispatchArgsCS SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->SuccessAndFailIndexBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SuccessAndFailIndexBuffer[0], PF_R32_UINT));
					PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(IndirectDispatchArgsBuffer, PF_R32_UINT));
				}

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				{
					ComputeShaderPermutationVector.Set<SHADER::FDimInputMatrixType>(RegressionKernel::EInputMatrixType::Fail);
				}

				TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NFOR::BuildIndirectDispatchCS"),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}

			typedef RegressionKernel::FLinearSolverIndirectCS SHADER;
			RegressionKernel::FLinearSolverCS::ESolverType PassSolverType = bApproximateGroundTruthSolver ?
				RegressionKernel::FLinearSolverCS::ESolverType::Cholesky : RegressionKernel::FLinearSolverCS::ESolverType::NewtonSchulz;
			const float LambdaExponent = -6.0f;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->CommonParameters = CommonPassParameters;
				PassParameters->CommonParameters.Lambda = FMath::Pow(10, LambdaExponent);
				if (bApproximateGroundTruthSolver)
				{
					PassParameters->CommonParameters.RWSuccessAndFailIndexBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(SuccessAndFailIndexBuffer[1], PF_R32_UINT));
				}
				PassParameters->SuccessAndFailIndexBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SuccessAndFailIndexBuffer[0], PF_R32_UINT));

				PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchArgsBuffer;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				checkf(BDim.X >= 6 && BDim.X <= 8, TEXT("Number of features should be between 6 and 8"));
				ComputeShaderPermutationVector.Set<SHADER::FDimNumFeature>(BDim.X);
				ComputeShaderPermutationVector.Set<SHADER::FDimSolverType>(PassSolverType);
				ComputeShaderPermutationVector.Set<SHADER::FDimInputMatrixType>(RegressionKernel::EInputMatrixType::Fail);
				ComputeShaderPermutationVector.Set<SHADER::FDimOutputIndices>(bApproximateGroundTruthSolver);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::BatchedLinearSolverIndirect(%s, Lambda=1e%.1f)", 
					GetLinearSolverTypeName(PassSolverType),
					LambdaExponent),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				PassParameters->IndirectDispatchArgsBuffer, 0);

		}

		// Early out for Newton Cholesky method.
		if (!bApproximateGroundTruthSolver)
		{
			return;
		}

		{
			//For failed indices, fallback to newton iterative method.
			{
				// Build the indirect dispatch parameters on failed
				typedef RegressionKernel::FLinearSolverBuildIndirectDispatchArgsCS SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->SuccessAndFailIndexBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SuccessAndFailIndexBuffer[1], PF_R32_UINT));
					PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(IndirectDispatchArgsBuffer, PF_R32_UINT));
				}

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				{
					ComputeShaderPermutationVector.Set<SHADER::FDimInputMatrixType>(RegressionKernel::EInputMatrixType::Fail);
				}

				TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NFOR::BuildIndirectDispatchCS"),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}

			typedef RegressionKernel::FLinearSolverIndirectCS SHADER;
			RegressionKernel::FLinearSolverCS::ESolverType PassSolverType = RegressionKernel::FLinearSolverCS::ESolverType::NewtonSchulz;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->CommonParameters = CommonPassParameters;
				PassParameters->SuccessAndFailIndexBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SuccessAndFailIndexBuffer[1], PF_R32_UINT));

				PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchArgsBuffer;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				checkf(BDim.X >= 6 && BDim.X <= 8, TEXT("Number of features should be between 6 and 8"));
				ComputeShaderPermutationVector.Set<SHADER::FDimNumFeature>(BDim.X);
				ComputeShaderPermutationVector.Set<SHADER::FDimSolverType>(PassSolverType);
				ComputeShaderPermutationVector.Set<SHADER::FDimInputMatrixType>(RegressionKernel::EInputMatrixType::Fail);
				ComputeShaderPermutationVector.Set<SHADER::FDimOutputIndices>(false);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::BatchedLinearSolverIndirect(%s on Failed)",
					GetLinearSolverTypeName(PassSolverType)),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				PassParameters->IndirectDispatchArgsBuffer, 0);
		}

		{
			// For succeeded indices, iteratively refine with smaller lambda.
			// Lambda = 10-7. TODO: ieratively refine.

			{
				// Build the indirect dispatch parameters on failed
				typedef RegressionKernel::FLinearSolverBuildIndirectDispatchArgsCS SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->SuccessAndFailIndexBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SuccessAndFailIndexBuffer[1], PF_R32_UINT));
					PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(IndirectDispatchArgsBuffer, PF_R32_UINT));
				}

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				{
					ComputeShaderPermutationVector.Set<SHADER::FDimInputMatrixType>(RegressionKernel::EInputMatrixType::Success);
				}

				TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NFOR::BuildIndirectDispatchCS"),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}

			const float LambdaExponent = -7.0f;
			typedef RegressionKernel::FLinearSolverIndirectCS SHADER;
			RegressionKernel::FLinearSolverCS::ESolverType PassSolverType = RegressionKernel::FLinearSolverCS::ESolverType::Cholesky;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->CommonParameters = CommonPassParameters;
				PassParameters->SuccessAndFailIndexBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SuccessAndFailIndexBuffer[1], PF_R32_UINT));
				PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchArgsBuffer;
				PassParameters->CommonParameters.Lambda = FMath::Pow(10, LambdaExponent);
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			{
				checkf(BDim.X >= 6 && BDim.X <= 8, TEXT("Number of features should be between 6 and 8"));
				ComputeShaderPermutationVector.Set<SHADER::FDimNumFeature>(BDim.X);
				ComputeShaderPermutationVector.Set<SHADER::FDimSolverType>(PassSolverType);
				ComputeShaderPermutationVector.Set<SHADER::FDimInputMatrixType>(RegressionKernel::EInputMatrixType::Success);
				ComputeShaderPermutationVector.Set<SHADER::FDimOutputIndices>(false);
			}

			TShaderMapRef<SHADER> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NFOR::BatchedLinearSolverIndirect(%s on Succeeded, Lambda=1e%.1f)",
					GetLinearSolverTypeName(PassSolverType),
					LambdaExponent),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				PassParameters->IndirectDispatchArgsBuffer, 0);
		}

	}

	void SolveWeightedLSR(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FRDGBufferRef& Feature,
		const FRDGTextureRef& Radiance,
		const FRDGBufferRef& NonLocalMeanWeightsBuffer,
		const FRDGTextureRef& FilteredRadiance,
		const FWeightedLSRDesc& WeightedLSRDesc,
		const FRDGBufferRef Radiances,
		const FRDGTextureRef& SourceAlbedo
	)
	{

		RDG_EVENT_SCOPE(GraphBuilder, "SolveWeightedLSR");

		FIntPoint TextureSize = Radiance->Desc.Extent;
		
		if (GetRegressionDevice() == ERegressionDevice::CPU)
		{
			SolveWeightedLSRCPU(
				GraphBuilder,
				View,
				Feature,
				Radiance,
				NonLocalMeanWeightsBuffer,
				FilteredRadiance,
				WeightedLSRDesc,
				Radiances,
				SourceAlbedo);

			return;
		}

		checkf(WeightedLSRDesc.SolverType == EWeightedLSRSolverType::Tiled,TEXT("The weighted LSR solver should select tiled"));

		FIntPoint XDimension = FIntPoint(WeightedLSRDesc.NumOfWeightsPerPixel, WeightedLSRDesc.NumOfFeatureChannelsPerFrame);
		int PatchDistance = (FMath::Sqrt(float(WeightedLSRDesc.NumOfWeightsPerPixel / WeightedLSRDesc.NumOfFrames)) - 1) / 2;

		// 1. Process the data into A, B for Ax=B.
		FRDGBufferRef AMatrix = nullptr;
		FRDGBufferRef BMatrix = nullptr;
		{
			AMatrix = ApplyBatchedInPlaceMatrixMultiplication(
				GraphBuilder,
				View,
				Feature,
				XDimension,
				NonLocalMeanWeightsBuffer,
				WeightedLSRDesc.NumOfWeightsPerPixel,
				FIntPoint(WeightedLSRDesc.Width, WeightedLSRDesc.Height),
				PatchDistance,
				RegressionKernel::EWeightedMultiplicationType::Quadratic);

			BMatrix = ApplyBatchedInPlaceMatrixMultiplication(
				GraphBuilder,
				View,
				Feature,
				XDimension,
				NonLocalMeanWeightsBuffer,
				WeightedLSRDesc.NumOfWeightsPerPixel,
				FIntPoint(WeightedLSRDesc.Width, WeightedLSRDesc.Height),
				PatchDistance,
				RegressionKernel::EWeightedMultiplicationType::Generalized,
				Radiances,
				FIntPoint(WeightedLSRDesc.NumOfWeightsPerPixel, WeightedLSRDesc.NumOfRadianceChannelsPerFrame));
		}

		// 2. Solve the linear equation Ax=B.
		const bool bFeatureAddConstant = ShouldFeatureAddConstant();
		const int32 NumOfAdditionalFeatures = bFeatureAddConstant ? 1 : 0;
		const int32 NumOfElementsPerRow = WeightedLSRDesc.Width;
		const int32 NumOfElements = WeightedLSRDesc.Width * WeightedLSRDesc.Height;
		const int32 TotalNumOfFeaturesPerFrame = WeightedLSRDesc.NumOfFeatureChannelsPerFrame + NumOfAdditionalFeatures;

		FIntPoint BDim = FIntPoint(TotalNumOfFeaturesPerFrame, WeightedLSRDesc.NumOfRadianceChannelsPerFrame);
		FRDGBufferRef ReconstructionWeights = RegressionKernel::AllocateMatrixfBuffer(
			GraphBuilder, 
			NumOfElements,
			BDim.X,
			BDim.Y, TEXT("NFOR.WeightedLSR.ReconstructWeights"));

		if (GetLinearSolverDevice() == ELinearSolverDevice::CPU)
		{
			SolveLinearEquationCPU(
				GraphBuilder,
				AMatrix,
				BMatrix,
				NumOfElements,
				BDim,
				ReconstructionWeights);
		}
		else
		{
			ApplyLinearSolverGPU(
				GraphBuilder,
				AMatrix,
				BMatrix,
				BDim,
				ReconstructionWeights,
				WeightedLSRDesc,
				TotalNumOfFeaturesPerFrame,
				NumOfElements,
				NumOfElementsPerRow);
		}

		// 3. Reconstruct 
		{
			AddClearUAVPass(GraphBuilder,GraphBuilder.CreateUAV(FilteredRadiance), 0.0f, ERDGPassFlags::Compute);

			const int32 ReconstructDebugFrameIndex = GetReconstructionDebugFrameIndex();
			const bool IsReconstructDebugEnabled = ReconstructDebugFrameIndex >= 0;

			for (int32 FrameIndex = 0; FrameIndex < WeightedLSRDesc.NumOfFrames; ++FrameIndex)
			{
				if (IsReconstructDebugEnabled)
				{
					FrameIndex = FMath::Min(ReconstructDebugFrameIndex, WeightedLSRDesc.NumOfFrames - 1);
				}

				RegressionKernel::FReconstructSpatialTemporalImage::EReconstructionType
					ReconstructionType = GetReconstructionType(FrameIndex, GetDenoisingFrameIndex(View,WeightedLSRDesc.NumOfFrames));
				
				ReconstructByFrame(
					GraphBuilder,
					Feature,
					ReconstructionWeights,
					NonLocalMeanWeightsBuffer,
					FilteredRadiance,
					SourceAlbedo,
					WeightedLSRDesc,
					FrameIndex,
					ReconstructionType);

				if (IsReconstructDebugEnabled)
				{
					break;
				}
			}
		}

		// Multiply albedo if the albedo recover phase is final.
		if (GetPreAlbedoDivideRecoverPhase()== EAlbedoDivideRecoverPhase::Final)
		{
			FIntPoint SourcePosition = WeightedLSRDesc.TileStartPosition - WeightedLSRDesc.Offset;
			AddMultiplyTextureRegionPass(GraphBuilder, SourceAlbedo, FilteredRadiance, true, SourcePosition, FIntPoint::ZeroValue, WeightedLSRDesc.TextureSize);
		}
	}

	int32 GetNumOfCombinedFeatureChannels(const TArray<FFeatureDesc>& FeatureDescs) {
		int NumOfChannels = 0;
		for (FFeatureDesc FeatureDesc : FeatureDescs)
		{
			if (FeatureDesc.Feature.Image)
			{
				NumOfChannels += FeatureDesc.Feature.ChannelCount;
			}
		}
		return NumOfChannels;
	};

	FRDGTextureRef CollaborativeRegression(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const TArray<FRadianceDesc>& Radiances,
		const TArray<FFeatureDesc>& Features,
		const FNonLocalMeanParameters& RadianceNonLocalMeanParameters)
	{
		const int32 NumOfFeatures = Features.Num();
		const int32 NumOfRadiances = Radiances.Num();
		const int32 SourceIndex = GetDenoisingFrameIndex(View,NumOfRadiances); // The current denoising frame data index

		const int SearchingPatchSize = (RadianceNonLocalMeanParameters.PatchDistance * 2 + 1);
		const int NumberOfWeightsPerPixel = SearchingPatchSize * SearchingPatchSize;

		// TODO: adaptive tile size for best performance.
		FIntPoint TextureSize = Radiances[0].Data.Image->Desc.Extent;
		const int32 NumOfTilesOneSide = GetNumOfTiles(TextureSize);
		FIntPoint NumOfTiles = FIntPoint(NumOfTilesOneSide, NumOfTilesOneSide);
		const int32 TotalTileCount = NumOfTilesOneSide * NumOfTilesOneSide;

		FIntPoint TileSize = FMath::DivideAndRoundUp(TextureSize, NumOfTiles);
		FIntPoint PaddingTileOffset = RadianceNonLocalMeanParameters.PatchDistance;
		FIntPoint PaddedTileSize = TileSize + PaddingTileOffset * 2;
		FIntRect  PaddedTileRect = FIntRect(FIntPoint(0,0), PaddedTileSize);
		const int NumOfCombinedFeatureChannels = GetNumOfCombinedFeatureChannels(Features);

		const int32 NonLocalMeanSingleFrameWeightSize = GetNonLocalMeanSingleFrameWeightBufferSize(TileSize, NumberOfWeightsPerPixel);
		const int32 BytesPerElement = sizeof(float);
		FRDGBufferDesc NonLocalMeanSingleFrameWeightsBufferDesc = 
			FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NonLocalMeanSingleFrameWeightSize);
		FRDGBufferRef NonLocalMeanSingleFrameWeightsBuffer = GraphBuilder.CreateBuffer(NonLocalMeanSingleFrameWeightsBufferDesc, TEXT("NFOR.NLMSingleFrameWeightsBuffer"));
		
		FRDGBufferRef NonLocalMeanWeightsBuffer = nullptr;
		if (NumOfRadiances > 1)
		{
			FRDGBufferDesc NonLocalFrameWeightsBufferDesc =
				FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NonLocalMeanSingleFrameWeightSize * NumOfRadiances);
			NonLocalMeanWeightsBuffer = GraphBuilder.CreateBuffer(NonLocalFrameWeightsBufferDesc, TEXT("NFOR.NLMWeightsBuffer"));
		}
		else
		{
			NonLocalMeanWeightsBuffer = NonLocalMeanSingleFrameWeightsBuffer;
		}

		FRDGBufferDesc CombinedFeatureDesc = FRDGBufferDesc::CreateBufferDesc(GetFeatureBytesPerElement(), PaddedTileSize.X * PaddedTileSize.Y * NumOfCombinedFeatureChannels);
		FRDGBufferRef CombinedFeatures = GraphBuilder.CreateBuffer(CombinedFeatureDesc, TEXT("NFOR.CombinedFeatures"));

		const int32 NumOfCombinedRadianceChannels = GetNumOfCombinedFeatureChannels(Radiances);
		FRDGBufferDesc CombinedRadianceDesc = FRDGBufferDesc::CreateBufferDesc(BytesPerElement * 4, PaddedTileSize.X * PaddedTileSize.Y * NumOfRadiances);
		FRDGBufferRef CombinedRadiances = GraphBuilder.CreateBuffer(CombinedRadianceDesc, TEXT("NFOR.CombinedRadiances"));

		FRDGTextureDesc FilteredRadianceDesc = Radiances[SourceIndex].Data.Image->Desc;
		{
			FilteredRadianceDesc.Flags |= TexCreate_RenderTargetable;
		
			if(FilteredRadianceDesc.Format == EPixelFormat::PF_FloatRGBA)
			{
				FilteredRadianceDesc.Format = EPixelFormat::PF_A32B32G32R32F;// The accumulation can run more than 2^16.
			}
		}
		FRDGTextureRef FilteredRadiance = GraphBuilder.CreateTexture(FilteredRadianceDesc, TEXT("NFOR.FilteredRadiance"));

		FRDGTextureDesc RadianceTileDesc = FilteredRadianceDesc;
		RadianceTileDesc.Extent = PaddedTileSize;
		FRDGTextureRef RadianceTileTexture = GraphBuilder.CreateTexture(RadianceTileDesc, TEXT("NFOR.RadianceTile"));
		FRDGTextureRef DenoisedTileTexture = GraphBuilder.CreateTexture(RadianceTileDesc, TEXT("NFOR.DenoisedRadianceTile"));
		
		RDG_EVENT_SCOPE(GraphBuilder, "CollaborativeRegression (bandwidth=%.2f)", RadianceNonLocalMeanParameters.Bandwidth);

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(FilteredRadiance), FLinearColor::Transparent, ERDGPassFlags::Compute);
		
		auto GetTileIndex = [TotalTileCount, NumOfTilesOneSide](int32 Index)
		{
			// TODO: generate tiles first and then iterate on tiles?
			if (IsTileDebugEnabled())
			{
				if (GetTileDebugIndex() < 0)
				{
					Index = TotalTileCount / 2 - NumOfTilesOneSide / 2;
				}
				else
				{
					Index = GetTileDebugIndex();
				}
			}
			return Index;
		};

		//TODO: Each tile can be parallelized.
		for (int32 i = 0; i < TotalTileCount; ++i)
		{
			int32 TileIndex = GetTileIndex(i);

			FIntPoint TileStartPoint = FIntPoint(TileIndex % NumOfTiles.X, TileIndex / NumOfTiles.X) * TileSize;
			FIntRect TileRegion = FIntRect(FIntPoint(0), TileSize) + TileStartPoint;
			FIntRect  PaddedTileRegion = PaddedTileRect + TileStartPoint - PaddingTileOffset;

			RDG_EVENT_SCOPE(GraphBuilder, "Tile (Index=%d)", TileIndex);

			// Get the weights W
			{
				RDG_EVENT_SCOPE(GraphBuilder, "GetNLMWeights (T=%d)", NumOfRadiances);

				for (int32 RadianceId = 0; RadianceId < NumOfRadiances; ++RadianceId)
				{
					GetNLMWeights(
						GraphBuilder,
						View,
						Radiances[SourceIndex],
						Radiances[RadianceId],
						NonLocalMeanSingleFrameWeightsBuffer,
						TileRegion,
						RadianceNonLocalMeanParameters);

					if (NumOfRadiances > 1)
					{
						AddCopyBufferPass(GraphBuilder, NonLocalMeanWeightsBuffer, NonLocalMeanSingleFrameWeightSize * BytesPerElement * RadianceId,
							NonLocalMeanSingleFrameWeightsBuffer, 0, NonLocalMeanSingleFrameWeightSize * BytesPerElement);
					}
				}
			}

			// Get raw color Y
			{
				RDG_EVENT_SCOPE(GraphBuilder, "GetRadiances (T=%d)", NumOfRadiances);
				
				int32 BufferChannelOffset = 0;
				
				for (int32 RadianceId = 0; RadianceId < NumOfRadiances; ++RadianceId)
				{
					AddCopyMirroredTexturePass(GraphBuilder, Radiances[RadianceId].Data.Image, RadianceTileTexture, PaddedTileRegion.Min, FIntPoint::ZeroValue, PaddedTileSize);

					FNFORTextureDesc Texture = Radiances[RadianceId].Data;

					AddCopyTextureToBufferPass(GraphBuilder, Texture.Image, CombinedRadiances,
						Texture.ChannelOffset,
						Texture.ChannelCount,
						Texture.NumOfChannel,
						BufferChannelOffset,
						NumOfCombinedRadianceChannels,
						PaddedTileRegion);

					BufferChannelOffset += Texture.ChannelCount;
				}

				checkf(BufferChannelOffset == NumOfCombinedRadianceChannels, TEXT("Number of channels used by radiances does not match the channel count in the buffer."));
			}

			// Get the feature vector X
			{
				RDG_EVENT_SCOPE(GraphBuilder, "GetFeatureVectors (TxF=%dx%d)", NumOfRadiances, NumOfFeatures / NumOfRadiances);

				int32 BufferChannelOffset = 0;

				for (int32 FeatureId = 0; FeatureId < NumOfFeatures; ++FeatureId)
				{
					FNFORTextureDesc Texture = Features[FeatureId].Data;

					AddCopyTextureToBufferPass(GraphBuilder, Texture.Image, CombinedFeatures,
						Texture.ChannelOffset,
						Texture.ChannelCount,
						Texture.NumOfChannel,
						BufferChannelOffset,
						NumOfCombinedFeatureChannels,
						PaddedTileRegion);

					BufferChannelOffset += Texture.ChannelCount;
				}

				checkf(BufferChannelOffset == NumOfCombinedFeatureChannels, TEXT("Number of channels used by feature does not match the channel count in the buffer."));
			}

			// Solve the weighted LSR.
			{
				FWeightedLSRDesc WeightedLSRDesc;
				{
					WeightedLSRDesc.NumOfFeatureChannels = NumOfCombinedFeatureChannels;
					WeightedLSRDesc.NumOfFeatureChannelsPerFrame = NumOfCombinedFeatureChannels / NumOfRadiances;
					WeightedLSRDesc.NumOfWeightsPerPixel = NumberOfWeightsPerPixel * NumOfRadiances;
					WeightedLSRDesc.NumOfWeightsPerPixelPerFrame = NumberOfWeightsPerPixel;
					WeightedLSRDesc.NumOfRadianceChannels = NumOfCombinedRadianceChannels;
					WeightedLSRDesc.NumOfRadianceChannelsPerFrame = NumOfCombinedRadianceChannels / NumOfRadiances;

					WeightedLSRDesc.Width = TileSize.X;
					WeightedLSRDesc.Height = TileSize.Y;
					WeightedLSRDesc.Offset = PaddingTileOffset;
					WeightedLSRDesc.TileStartPosition = TileStartPoint;
					WeightedLSRDesc.NumOfFrames = NumOfRadiances;
					WeightedLSRDesc.TextureSize = RadianceTileTexture->Desc.Extent;
					WeightedLSRDesc.SolverType = EWeightedLSRSolverType::Tiled;
				}

				const int SourceAlbedoFeatureIndex = (NumOfFeatures / NumOfRadiances) * SourceIndex;

				SolveWeightedLSR(
					GraphBuilder,
					View,
					CombinedFeatures,
					RadianceTileTexture,
					NonLocalMeanWeightsBuffer,
					DenoisedTileTexture,
					WeightedLSRDesc,
					CombinedRadiances,
					Features[SourceAlbedoFeatureIndex].Data.Image);
			}

			// Copy back and accumulate.
			AddAccumulateTextureRegionPass(GraphBuilder, DenoisedTileTexture, FilteredRadiance, FIntPoint::ZeroValue, PaddedTileRegion.Min, PaddedTileSize);

			if (IsTileDebugEnabled())
			{
				break;
			}
		}

		// Normalize the image by weights stored in alpha channel.
		AddNormalizeTexturePass(GraphBuilder, FilteredRadiance);

		// Copy back with the original format.
		{
			FRDGTextureDesc FilteredRadianceOutputDesc = Radiances[SourceIndex].Data.Image->Desc;
			if (FilteredRadianceOutputDesc.Format == EPixelFormat::PF_FloatRGBA)
			{
				FRDGTextureRef FilteredRadianceOutputTexture = GraphBuilder.CreateTexture(FilteredRadianceOutputDesc, TEXT("NFOR.FilteredRadiance.Output"));
				AddCopyMirroredTexturePass(GraphBuilder, FilteredRadiance, FilteredRadianceOutputTexture);
				FilteredRadiance = FilteredRadianceOutputTexture;
			}
		}

		return FilteredRadiance;
	}

	//--------------------------------------------------------------------------------------------------------------------
	// Bandwidth selection

	FNFORTextureDesc MSEEstimation(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FRadianceDesc& Radiance,
		const FRDGTextureRef FilteredImage)
	{
		FIntPoint TextureSize = Radiance.Data.Image->Desc.Extent;
		FRDGTextureDesc Desc = Radiance.Variance.Image->Desc;
		Desc.Format = PF_R32_FLOAT;
		FRDGTextureRef MSE = GraphBuilder.CreateTexture(Desc, TEXT("NFOR.MSE"));
		NFORDenoise::FNFORTextureDesc NFORMSETexture(MSE, 0, 1, 1);

		typedef FMSEEstimationCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->Variance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Radiance.Variance.Image));
			PassParameters->Image = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Radiance.Data.Image));
			PassParameters->FilteredImage = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(FilteredImage));
			PassParameters->TextureSize = TextureSize;
			PassParameters->VarianceChannelOffset = Radiance.Variance.ChannelOffset;
			PassParameters->MSE = GraphBuilder.CreateUAV(NFORMSETexture.Image);
		}

		SHADER::FPermutationDomain ComputeShaderPermutationVector;
		ComputeShaderPermutationVector.Set<SHADER::FDimensionVarianceType>(Radiance.VarianceType);
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap, ComputeShaderPermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NFOR::MSEEstimation (Dim=%d,%d)", TextureSize.X, TextureSize.Y),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(TextureSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));

		return NFORMSETexture;
	}

	//--------------------------------------------------------------------------------------------------------------------
	// NFOR filtering and denoising

	void FilterFeatures(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const TArray<FFeatureDesc>& FeatureDescs)
	{
		
		FNonLocalMeanParameters FeatureNonLocalMeanParameters = GetFeatureNonLocalMeanParameters(0.5f);

		const int NumOfFeatures = FeatureDescs.Num();

		int32 BufferChannelOffset = 0;
		for (int i = 0; i < NumOfFeatures; ++i)
		{
			if (FeatureDescs[i].Feature.Image == nullptr 
				|| FeatureDescs[i].bCleanFeature)
			{
				continue;
			}

			const FFeatureDesc& FeatureDesc = FeatureDescs[i];
			FRDGTextureRef Feature = FeatureDesc.Feature.Image;
			FRDGTextureRef FilterdFeature = GraphBuilder.CreateTexture(Feature->Desc, TEXT("NFOR.FilteredFeature"));

			ApplyNonLocalMeanFilterIfRequired(
				GraphBuilder,
				View,
				FeatureNonLocalMeanParameters,
				FeatureDesc.Feature,
				FeatureDesc.Variance,
				FeatureDesc.VarianceType,
				FilterdFeature,
				GetFeatureTileSizeDownScale());

			AddCopyTexturePass(GraphBuilder, FilterdFeature, Feature);
		}
	}

	bool FilterMain(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const TArray<FRadianceDesc>& Radiances,
		const TArray<FFeatureDesc>& FeatureDescs,
		const FRDGTextureRef& DenoisedRadiance
	)
	{
		// Frame 0, 1,...,n-1
		//         0: new frames with feature frame denoised with NLM.
		// 1,...,n-1: old frames with feature frame denoised with NLM.

		// Denoise for frame n/2. E.g., 
		//		when n=3, n_m=1,2nd frame is the current frame to denoise. 0, |1|, 2
		//		when n=5, n_m=2,3rd frame is the current frame to denoise. 0, 1, |2|, 3, 4
		// Special case when n_a < n
		//		n_a <= n/2: n_m=n_a
		//      n_a >  n/2: n_m=n/2
		// Since the frame size can be very large, we denoise tile by tile and resolve at last
		// 
		// Pseudo code:
		// 
		//  Preprocessing
		// 
		//	For each bandwidth:
		//		For each tile in tiles:
		//			collaborative regression(tile)
		//		denoised = recombine(tiles)
		// 
		//  Bandwidth Selection
  
		const int32 NumOfTemporalFrames = Radiances.Num();
		const int32 NumOfFeatures = FeatureDescs.Num();
		const int32 NumOfFeaturesPerFrame = NumOfFeatures / NumOfTemporalFrames;
		const int32 SourceRadianceIndex = GetDenoisingFrameIndex(View,NumOfTemporalFrames);
		
		// Preprocessing
		// Feature range adjustment, radiance normalization and filtering frames
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Preprocessing");

			// Latest frame only 
			for (int i = 0; i < 1; ++i)
			{
				const FFeatureDesc& Albedo = FeatureDescs[i * NumOfFeaturesPerFrame + 0]; //TODO: unify the index
				const FFeatureDesc& Normal = FeatureDescs[i * NumOfFeaturesPerFrame + 1]; //TODO: unify the index

				// Adjust feature range if required
				AddAdjustFeatureRangePass(GraphBuilder, Albedo, GetFeatureMaxAlbedoGrayscale());
				AddAdjustFeatureRangePass(GraphBuilder, Normal, GetFeatureMaxNormalLength());

				if (IsPreAlbedoDivideEnabled())
				{
					FRDGTextureRef AlbedoTex = Albedo.Feature.Image; 
					FRDGTextureRef NormalTex = Normal.Feature.Image; 
					FRDGTextureRef NormalVarianceTex = Normal.Variance.Image;
					FRDGTextureRef MaskTexture = NFORDenoise::GetPreAlbedoDivideMask(GraphBuilder, View, NormalTex, NormalVarianceTex);

					FLinearColor RGBOffset = NFORDenoise::GetPreAlbedoDivideAlbedoOffset();

					NFORDenoise::AddAccumulateConstantRegionPass(GraphBuilder, RGBOffset, AlbedoTex, MaskTexture);

					FRDGTextureRef RadianceTexture = Radiances[i].Data.Image;
					FRDGTextureRef RadianceVarianceTexture = Radiances[i].Variance.Image;

					// Normalization should apply to both texture and variance.
					NFORDenoise::AddDivideTextureRegionPass(GraphBuilder, AlbedoTex, RadianceTexture);
					NFORDenoise::AddNormalizeRadianceVariancePass(GraphBuilder, AlbedoTex, RadianceVarianceTexture);
				}
			}

			TArray<FFeatureDesc> LatestFrameFeature = TArray<FFeatureDesc>(FeatureDescs.GetData(), NumOfFeaturesPerFrame);
			NFORDenoise::FilterFeatures(GraphBuilder, View, LatestFrameFeature);
		}

		{	// Early out if radiance denoising is not required.
			EDenoiseFrameCountCondition Condition = GetFrameCountCoundition();

			if (Condition == EDenoiseFrameCountCondition::Equal && 
				SourceRadianceIndex == INDEX_NONE)
			{
				return false;
			}
		}

		TArray<FRDGTextureRef> FilteredImages = {};
		TArray<FNFORTextureDesc> FilteredMSEs = {};
		TArray<float> Bandwidths = GetBandwidthsConfiguration();

		const bool bPerformBandwidthSelection = IsBandwidthSelectionEnabled() && Bandwidths.Num() == 2;
		const FRadianceDesc& SourceRadiance = Radiances[SourceRadianceIndex];
		const int32 RadiancePatchSize = GetNonLocalMeanRadiancePatchSize();
		const int32 RadiancePatchDistance = GetNonLocalMeanRadiancePatchDistance();

		for (int i = 0; i < Bandwidths.Num(); ++i)
		{
			// Collaborative regression.
			FNonLocalMeanParameters RadianceNonLocalMeanParameters = GetNonLocalMeanParameters(RadiancePatchSize, RadiancePatchDistance, Bandwidths[i]);
			FRDGTextureRef FilteredImage = CollaborativeRegression(GraphBuilder, View, Radiances, FeatureDescs, RadianceNonLocalMeanParameters);

			FilteredImages.Add(FilteredImage);

			// MSE estimation and filtering.
			if (bPerformBandwidthSelection)
			{		
				FNFORTextureDesc MSE = MSEEstimation(GraphBuilder, View, SourceRadiance, FilteredImage);

				// NLM filtering of MSE texture.
				FRDGTextureRef FilteredMSETexure = GraphBuilder.CreateTexture(MSE.Image->Desc, TEXT("NFOR.FilteredMSE"));
				FNonLocalMeanParameters MSENonLocalMeanParameters = GetNonLocalMeanParameters(1, RadiancePatchDistance, 
					ShouldBandwidthSelectionMSEPreserveDetail() ? Bandwidths[i]: 1.0f);

				ApplyNonLocalMeanFilterIfRequired(
					GraphBuilder,
					View,
					MSENonLocalMeanParameters,
					MSE,
					SourceRadiance.Variance,
					SourceRadiance.VarianceType,
					FilteredMSETexure);

				FNFORTextureDesc FilterdMSE = MSE;
				FilterdMSE.Image = FilteredMSETexure;

				FilteredMSEs.Add(FilterdMSE);
			}
		}
		
		if (bPerformBandwidthSelection)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "BandwidthSelection");

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			FRDGTextureDesc Desc = FilteredMSEs[0].Image->Desc;
			FNFORTextureDesc NFORSelectionMap = FilteredMSEs[0];
			NFORSelectionMap.Image = GraphBuilder.CreateTexture(Desc, TEXT("NFOR.SelectionMap"));
			FIntPoint TextureSize = Desc.Extent;
			{
				typedef FGenerateSelectionMapCS SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->FilteredMSEs[0] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(FilteredMSEs[0].Image));
					PassParameters->FilteredMSEs[1] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(FilteredMSEs[1].Image));
					PassParameters->TextureSize = TextureSize;
					PassParameters->RWSelectionMap = GraphBuilder.CreateUAV(NFORSelectionMap.Image);
				}

				TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NFOR::GenerateSelectionMap (Dim=%d,%d)", TextureSize.X, TextureSize.Y),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(TextureSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
			}

			// Filter the selection map with image variance
			FRDGTextureRef FilteredSelectionMap = GraphBuilder.CreateTexture(Desc, TEXT("NFOR.FilteredSelectionMap"));
			{
				FNonLocalMeanParameters SelectionMapNonLocalMeanParameters = GetNonLocalMeanParameters(1, RadiancePatchDistance, 
					ShouldBandwidthSelectionMapPreserveDetail() ? Bandwidths[0] : 1.0f);

				ApplyNonLocalMeanFilterIfRequired(
					GraphBuilder,
					View,
					SelectionMapNonLocalMeanParameters,
					NFORSelectionMap,
					SourceRadiance.Variance,
					SourceRadiance.VarianceType,
					FilteredSelectionMap);

				// Combine the filtered images.
				{
					typedef FCombineFilteredImageCS SHADER;
					SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
					{
						PassParameters->FilteredImages[0] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(FilteredImages[0]));
						PassParameters->FilteredImages[1] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(FilteredImages[1]));
						PassParameters->SelectionMap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(FilteredSelectionMap));
						PassParameters->TextureSize = TextureSize;
						PassParameters->RWFilteredImage = GraphBuilder.CreateUAV(DenoisedRadiance);
					}

					TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("NFOR::ApplySelectionMap (Dim=%d,%d)", TextureSize.X, TextureSize.Y),
						ERDGPassFlags::Compute,
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(TextureSize, NON_LOCAL_MEAN_THREAD_GROUP_SIZE));
				}
			}

			// Second regression pass is ignored as there is only one buffer.
		}
		else
		{
			// Support different 4 channel format texture copy
			AddCopyMirroredTexturePass(GraphBuilder, FilteredImages[0], DenoisedRadiance);
		}

		{
			RDG_EVENT_SCOPE(GraphBuilder, "Postprocessing");

			// Alpha denoising
			{
				FRDGTextureRef RadianceAlpha = Radiances[SourceRadianceIndex].Data.Image;
				if (ShouldDenoiseAlpha())
				{
					// Apply non-local mean filter to alpha only. 
					FRDGTextureRef RadianceAlphaVariance = Radiances[SourceRadianceIndex].Variance.Image;
					EVarianceType VarianceType = Radiances[SourceRadianceIndex].VarianceType;
					FRDGTextureDesc AlphaTextureDesc = RadianceAlpha->Desc;
					AlphaTextureDesc.Format = PF_R32_FLOAT;
					FRDGTextureRef RawAlphaTexture = GraphBuilder.CreateTexture(AlphaTextureDesc, TEXT("NFOR.RawAlphaTexture"));
					FRDGTextureRef FilteredAlphaTexture = GraphBuilder.CreateTexture(AlphaTextureDesc, TEXT("NFOR.DenoisedAlphaTexture"));

					// Assume alpha is the a component of Radiance texture, the alpha variance is the a component of the corresponding variance texture.
					const int32 AlphaChannelIndex = 3;

					AddCopyMirroredTexturePass(GraphBuilder, RadianceAlpha, RawAlphaTexture, AlphaChannelIndex, ETextureCopyType::TargetSingleChannel);

					FNFORTextureDesc AlphaTexture = FNFORTextureDesc(RawAlphaTexture, 0, 1, 1);
					FNonLocalMeanParameters AlphaNLMParams = GetFeatureNonLocalMeanParameters(0.5f);
					FNFORTextureDesc AlphaVariance = FNFORTextureDesc(RadianceAlphaVariance, AlphaChannelIndex, 1, 4);

					ApplyNonLocalMeanFilterIfRequired(
						GraphBuilder,
						View,
						AlphaNLMParams,
						AlphaTexture,
						AlphaVariance,
						VarianceType,
						FilteredAlphaTexture,
						GetFeatureTileSizeDownScale());

					AddCopyMirroredTexturePass(GraphBuilder, FilteredAlphaTexture, DenoisedRadiance, AlphaChannelIndex, ETextureCopyType::SourceSingleChannel);
				}
				else
				{
					// Pass through alpha channel.
					AddCopyMirroredTexturePass(GraphBuilder, RadianceAlpha, DenoisedRadiance, FIntPoint::ZeroValue, FIntPoint::ZeroValue, FIntPoint::ZeroValue, true/*bAlphaOnly*/);
				}
			}
		}

		return true;
	}
}
