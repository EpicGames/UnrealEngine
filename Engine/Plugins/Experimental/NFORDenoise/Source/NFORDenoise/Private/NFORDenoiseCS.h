// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "ShaderCompilerCore.h"

#define NON_LOCAL_MEAN_THREAD_GROUP_SIZE 8

#define TEXTURE_OPS_MULTIPLY			0
#define TEXTURE_OPS_DIVIDE				1
#define TEXTURE_OPS_ADD_CONSTANT		2
#define TEXTURE_OPS_ACCUMULATE			3

namespace NFORDenoise
{

	int32 GetFrameCount(const FSceneView& View);
	int32 GetDenoisingFrameIndex(const FSceneView& View, int32 NumberOfFrameInBuffer = -1);

	enum class EVarianceType : uint32
	{
		Normal,		// length of vector
		GreyScale,	// grey scale
		Colored,	// Not supported at this moment. 
		MAX
	};

	enum class EImageChannelCount : uint32
	{
		One,
		Two,
		Three,
		Four,
		MAX
	};

	enum class ENonLocalMeanAtlasType : uint32
	{
		OneSymmetricPair,
		TwoSymmetricPair,
		MAX
	};

	enum class EAlbedoDivideRecoverPhase : uint32
	{
		Disabled,
		Each,
		Final,
		MAX
	};
	

	//--------------------------------------------------------------------------------------------------------------------
	// General texture operations including: multiply, divide, accumulate, and copy.

	bool ShouldCompileNFORShadersForProject(EShaderPlatform ShaderPlatform);

	class FTextureMultiplyCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FTextureMultiplyCS)
		SHADER_USE_PARAMETER_STRUCT(FTextureMultiplyCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Source)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWTarget)
			SHADER_PARAMETER(FIntPoint, SourcePosition)
			SHADER_PARAMETER(FIntPoint, TargetPosition)
			SHADER_PARAMETER(FIntPoint, Size)
			SHADER_PARAMETER(int32, ForceOperation)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.SetDefine(TEXT("TEXTURE_OPS"), TEXTURE_OPS_MULTIPLY);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	class FTextureDivideCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FTextureDivideCS)
		SHADER_USE_PARAMETER_STRUCT(FTextureDivideCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Source)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWTarget)
			SHADER_PARAMETER(FIntPoint, SourcePosition)
			SHADER_PARAMETER(FIntPoint, TargetPosition)
			SHADER_PARAMETER(FIntPoint, Size)
			SHADER_PARAMETER(int32, ForceOperation)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.SetDefine(TEXT("TEXTURE_OPS"), TEXTURE_OPS_DIVIDE);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	class FTextureAccumulateConstantCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FTextureAccumulateConstantCS)
		SHADER_USE_PARAMETER_STRUCT(FTextureAccumulateConstantCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWTarget)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Mask)
			SHADER_PARAMETER(FIntPoint, TargetPosition)
			SHADER_PARAMETER(FIntPoint, Size)
			SHADER_PARAMETER(FLinearColor, ConstantValue)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.SetDefine(TEXT("TEXTURE_OPS"), TEXTURE_OPS_ADD_CONSTANT);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionAccumulateByMask : SHADER_PERMUTATION_BOOL("ACCUMULATE_BY_MASK");
		using FPermutationDomain = TShaderPermutationDomain<FDimensionAccumulateByMask>;
	};

	// Accumulate all channels of a texture onto another texture for a given region.
	class FTextureAccumulateCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FTextureAccumulateCS)
		SHADER_USE_PARAMETER_STRUCT(FTextureAccumulateCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Source)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWTarget)
			SHADER_PARAMETER(FIntPoint, SourcePosition)
			SHADER_PARAMETER(FIntPoint, TargetPosition)
			SHADER_PARAMETER(FIntPoint, Size)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.SetDefine(TEXT("TEXTURE_OPS"), TEXTURE_OPS_ACCUMULATE);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	class FCopyTexturePS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FCopyTexturePS)
		SHADER_USE_PARAMETER_STRUCT(FCopyTexturePS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Source)
			SHADER_PARAMETER(FIntPoint, SourceOffset)
			SHADER_PARAMETER(FIntPoint, TextureSize)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	enum class ETextureCopyType :uint32
	{
		TargetSingleChannel,
		SourceSingleChannel,
		MAX
	};

	class FCopyTextureSingleChannelCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FCopyTextureSingleChannelCS)
		SHADER_USE_PARAMETER_STRUCT(FCopyTextureSingleChannelCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, CopySource)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWCopyTarget)
			SHADER_PARAMETER(FIntPoint, SourceOffset)
			SHADER_PARAMETER(FIntPoint, TargetOffset)
			SHADER_PARAMETER(FIntPoint, CopySize)
			SHADER_PARAMETER(FIntPoint, TextureSize)
			SHADER_PARAMETER(int32, Channel)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimTextureCopyType : SHADER_PERMUTATION_ENUM_CLASS("TEXTURE_COPY_TYPE", ETextureCopyType);
		using FPermutationDomain = TShaderPermutationDomain<FDimTextureCopyType>;
	};

	// TargetTexture.rgb = TargetTexture.rgb * lerp (1.0f,SourceTexture.rgb, (SourceTexture.rgb != 0 || bForceMultiply));
	void AddMultiplyTextureRegionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		bool bForceMultiply = true, FIntPoint SourcePosition = FIntPoint::ZeroValue, FIntPoint TargetPosition = FIntPoint::ZeroValue, FIntPoint Size = FIntPoint::ZeroValue);

	// TargetTexture.rgb = TargetTexture.rgb / lerp (1.0f,SourceTexture.rgb, SourceTexture.rgb != 0 || bForceDivide);
	void AddDivideTextureRegionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		bool bForceDivide = true, FIntPoint SourcePosition = FIntPoint::ZeroValue, FIntPoint TargetPosition = FIntPoint::ZeroValue, FIntPoint Size = FIntPoint::ZeroValue);
	
	// TargetTexture += SourceTexture;
	void AddAccumulateTextureRegionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		FIntPoint SourcePosition = FIntPoint::ZeroValue, FIntPoint TargetPosition = FIntPoint::ZeroValue, FIntPoint Size = FIntPoint::ZeroValue);

	// TargetTexture.rgb = TargetTexture.rgb + lerp(ConstantValue, ConstantValue[mask], Mask != nullptr);
	void AddAccumulateConstantRegionPass(FRDGBuilder& GraphBuilder, const FLinearColor& ConstantValue, const FRDGTextureRef& TargetTexture, 
		const FRDGTextureRef& Mask = nullptr, FIntPoint SourcePosition = FIntPoint::ZeroValue, FIntPoint TargetPosition = FIntPoint::ZeroValue, FIntPoint Size = FIntPoint::ZeroValue);

	// Copy texture with mirrored border of source texture;
	void AddCopyMirroredTexturePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture,
		FIntPoint SourcePosition = FIntPoint::ZeroValue, FIntPoint TargetPosition = FIntPoint::ZeroValue, FIntPoint Size = FIntPoint::ZeroValue, bool bAlphaOnly = false);
	
	// Copy a channel from the source image to the target image where ETextureCopyType indicates which image is single channel;
	void AddCopyMirroredTexturePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, const FRDGTextureRef& TargetTexture, int32 Channel, ETextureCopyType CopyType,
		FIntPoint SourcePosition = FIntPoint::ZeroValue, FIntPoint TargetPosition = FIntPoint::ZeroValue, FIntPoint Size = FIntPoint::ZeroValue);

	//--------------------------------------------------------------------------------------------------------------------
	// Feature range adjustment, and radiance normalization by albedo
	
	class FClassifyPreAlbedoDivideMaskIdCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FClassifyPreAlbedoDivideMaskIdCS);
		SHADER_USE_PARAMETER_STRUCT(FClassifyPreAlbedoDivideMaskIdCS, FGlobalShader);
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Normal)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, NormalVariance)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWMask)

			SHADER_PARAMETER(FIntPoint, TextureSize)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	// Based on Taylor expansion, sigma_{normalized radiance} \approx = sigma_{radiance} / sigma_{albedo}, if normalized radiance = radiance / albedo
	class FNormalizeRadianceVarianceByAlbedoCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNormalizeRadianceVarianceByAlbedoCS)
			SHADER_USE_PARAMETER_STRUCT(FNormalizeRadianceVarianceByAlbedoCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, Albedo)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWRadianceVariance)
			SHADER_PARAMETER(FIntPoint, Size)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	struct FFeatureDesc;

	// Remap feature range
	class FAdjustFeatureRangeCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FAdjustFeatureRangeCS)
		SHADER_USE_PARAMETER_STRUCT(FAdjustFeatureRangeCS, FGlobalShader)
	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWImage)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWImageVariance)
			SHADER_PARAMETER(FIntPoint, Size)
			SHADER_PARAMETER(int32, VarianceChannelOffset)
			SHADER_PARAMETER(float, MaxValue)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionVarianceType : SHADER_PERMUTATION_ENUM_CLASS("IMAGE_VARIANCE_TYPE", EVarianceType);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionVarianceType>;
	};

	FRDGTextureRef GetPreAlbedoDivideMask(FRDGBuilder& GraphBuilder, const FSceneView& View, const FRDGTextureRef& Normal, const FRDGTextureRef& NormalVariance);

	void AddNormalizeRadianceVariancePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& Albedo, const FRDGTextureRef& RadianceVariance);

	// Adjust feature and variance range based on MaxValue to suppress random specular noise.
	// NewValue = min(Value, MaxValue). Std = lerp(Std,(MaxValue/Value)*Std,Value > MaxValue).
	void AddAdjustFeatureRangePass(FRDGBuilder& GraphBuilder, const FFeatureDesc& FeatureDesc, float MaxValue);

	//--------------------------------------------------------------------------------------------------------------------
	// Non-local mean weight and filtering
	
	BEGIN_SHADER_PARAMETER_STRUCT(FNonLocalMeanParameters, )
		SHADER_PARAMETER(int32, PatchSize)
		SHADER_PARAMETER(int32, PatchDistance)
		SHADER_PARAMETER(float, Bandwidth)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FNonLocalMeanWeightAtlasDispatchParameters, )
		SHADER_PARAMETER(int32, DispatchId)
		SHADER_PARAMETER(FIntPoint, DispatchTileSize)
		SHADER_PARAMETER(int32, DispatchTileCount)
		SHADER_PARAMETER(FIntRect, SeparableFilteringRegion)
		SHADER_PARAMETER(FIntVector, DispatchRegionSize)
	END_SHADER_PARAMETER_STRUCT()

	FNonLocalMeanParameters GetNonLocalMeanParameters(int32 PatchSize, int32 PatchDistance, float Bandwidth);

	enum class ENonLocalMeanWeightLayout : uint32
	{
		/**Weight buffer is not in use*/
		None,
		NumOfWeightsPerPixelxWxH,
		WxHxNumOfWeightsPerPixel,
		Float4xWxHxNumOfWeightsPerPixelByFloat4,
		MAX
	};

	struct FNonLocalMeanWeightDesc
	{
		/** The region this weights is gathered for.*/
		FIntRect Region;

		/** The gathered weights*/
		FRDGBufferRef WeightBuffer = nullptr;

		/** The layout of the weight*/
		ENonLocalMeanWeightLayout WeightLayout = ENonLocalMeanWeightLayout::None;
	};

	// Output the non-local mean filtered image (feature) based on variance
	// Input: Image, variance, Guide(optional), NonLocalMean parameters
	//		 Dim: WxHx?, WxHx?
	// Output: Image holding  prefiltered features
	//		 Dim: WxHx?
	class FNonLocalMeanFilteringCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNonLocalMeanFilteringCS)
		SHADER_USE_PARAMETER_STRUCT(FNonLocalMeanFilteringCS, FGlobalShader)
	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanParameters, NLMParams)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, Guide)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Variance)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Image)
			SHADER_PARAMETER(FIntPoint, TextureSize)
			SHADER_PARAMETER(int32, VarianceChannelOffset)
			SHADER_PARAMETER(int32, DenoisingChannelCount)
			SHADER_PARAMETER(FIntRect, FilteringRegion)

			//If using non-local mean weights for filtering acceleration
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, NonLocalMeanWeights)

			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DenoisedImage)
			END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.SetDefine(TEXT("NONLOCALMEAN_SEPARATE_SOURCE"), 0);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionVarianceType : SHADER_PERMUTATION_ENUM_CLASS("IMAGE_VARIANCE_TYPE", EVarianceType);
		class FDimensionUseGuide : SHADER_PERMUTATION_BOOL("USE_GUIDE"); //TODO
		class FDimensionImageChannelCount : SHADER_PERMUTATION_RANGE_INT("SOURCE_CHANNEL_COUNT", 1, static_cast<int>(EImageChannelCount::MAX));
		class FDimPreAlbedoDivide : SHADER_PERMUTATION_ENUM_CLASS("PRE_ALBEDO_DIVIDE", EAlbedoDivideRecoverPhase);
		class FDimWeightLayout : SHADER_PERMUTATION_ENUM_CLASS("NLM_WEIGHTLAYOUT", ENonLocalMeanWeightLayout);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionVarianceType, FDimensionUseGuide, FDimensionImageChannelCount, FDimPreAlbedoDivide, FDimWeightLayout>;
	};


	// Output the non-local mean weights for each pixel, used to solve the weighted least squares problem.
	// Input: Image, variance, NonLocalMean parameters
	//		 Dim: WxHx3, WxHx?
	// Output: Buffer holding  weights for each pixel
	//		 Dim: WxHx(2*N+1)^2, where N = NLMParams.PatchDistance
	class FNonLocalMeanWeightsCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNonLocalMeanWeightsCS)
		SHADER_USE_PARAMETER_STRUCT(FNonLocalMeanWeightsCS, FGlobalShader)
	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanFilteringCS::FParameters, CommonParameters)
			SHADER_PARAMETER(FIntRect, Region)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetImage)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetVariance)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, RWNonLocalMeanWeights)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionVarianceType : SHADER_PERMUTATION_ENUM_CLASS("IMAGE_VARIANCE_TYPE", EVarianceType);
		class FDimensionUseGuide : SHADER_PERMUTATION_BOOL("USE_GUIDE");
		class FDimensionImageChannelCount : SHADER_PERMUTATION_RANGE_INT("SOURCE_CHANNEL_COUNT", 1, static_cast<int>(EImageChannelCount::MAX));
		class FDimensionSeparateSourceTarget : SHADER_PERMUTATION_BOOL("NONLOCALMEAN_SEPARATE_SOURCE");
		class FDimPreAlbedoDivide : SHADER_PERMUTATION_ENUM_CLASS("PRE_ALBEDO_DIVIDE", EAlbedoDivideRecoverPhase);
		class FDimTargetWeightLayout : SHADER_PERMUTATION_ENUM_CLASS("NLM_WEIGHTLAYOUT", ENonLocalMeanWeightLayout);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionVarianceType, FDimensionUseGuide, FDimensionImageChannelCount, FDimensionSeparateSourceTarget, 
			FDimPreAlbedoDivide, FDimTargetWeightLayout>;
	};

	// Optimize the weights query.
	class FNonLocalMeanGetSqauredDistanceToAtlasCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNonLocalMeanGetSqauredDistanceToAtlasCS)
		SHADER_USE_PARAMETER_STRUCT(FNonLocalMeanGetSqauredDistanceToAtlasCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanFilteringCS::FParameters, CommonParameters)
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanWeightAtlasDispatchParameters, NLMWeightAtlasDispatchParameters)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetImage)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetVariance)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWNLMWeightAtlas)
			SHADER_PARAMETER(FIntPoint, NLMWeightAtlasSize)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionVarianceType : SHADER_PERMUTATION_ENUM_CLASS("IMAGE_VARIANCE_TYPE", EVarianceType);
		class FDimensionImageChannelCount : SHADER_PERMUTATION_RANGE_INT("SOURCE_CHANNEL_COUNT", 1, static_cast<int>(EImageChannelCount::MAX));
		class FDimensionSeparateSourceTarget : SHADER_PERMUTATION_BOOL("NONLOCALMEAN_SEPARATE_SOURCE");
		class FDimPreAlbedoDivide : SHADER_PERMUTATION_ENUM_CLASS("PRE_ALBEDO_DIVIDE", EAlbedoDivideRecoverPhase);
		class FDimAtlasType : SHADER_PERMUTATION_ENUM_CLASS("NONLOCALMEAN_ATLAS_TYPE", ENonLocalMeanAtlasType);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionVarianceType, FDimensionImageChannelCount, FDimensionSeparateSourceTarget, FDimPreAlbedoDivide, FDimAtlasType>;
	};

	class FNonLocalMeanSeperableFilterPatchSqauredDistanceCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNonLocalMeanSeperableFilterPatchSqauredDistanceCS)
		SHADER_USE_PARAMETER_STRUCT(FNonLocalMeanSeperableFilterPatchSqauredDistanceCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanParameters, NLMParams)
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanWeightAtlasDispatchParameters, NLMWeightAtlasDispatchParameters)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, NLMWeightAtlasSource)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWNLMWeightAtlasTarget)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWNLMWeights)
			SHADER_PARAMETER(FIntVector, SeperableRegionSize)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		enum class ESeperablePassType : int32
		{
			Horizontal,
			Vertical,
			MAX
		};

		class FDimensionSeperablePassType : SHADER_PERMUTATION_ENUM_CLASS("NONLOCALMEAN_SEPRERABLE_PASS", ESeperablePassType);
		class FDimPreAlbedoDivide : SHADER_PERMUTATION_ENUM_CLASS("PRE_ALBEDO_DIVIDE", EAlbedoDivideRecoverPhase);
		class FDimAtlasType : SHADER_PERMUTATION_ENUM_CLASS("NONLOCALMEAN_ATLAS_TYPE", ENonLocalMeanAtlasType);
		class FDimBufferPassThrough : SHADER_PERMUTATION_BOOL("BUFFER_PASS_THROUGH");
		using FPermutationDomain = TShaderPermutationDomain<FDimensionSeperablePassType, FDimPreAlbedoDivide, FDimAtlasType, FDimBufferPassThrough>;
	};

	// Reshape the layout of the buffer to target
	// From XxYx[W/B] (each element is of size B) to W*X*Y
	class FNonLocalMeanReshapeBufferCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNonLocalMeanReshapeBufferCS)
		SHADER_USE_PARAMETER_STRUCT(FNonLocalMeanReshapeBufferCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FNonLocalMeanParameters, NLMParams)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float2>, SourceBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, RWTargetBuffer)
			SHADER_PARAMETER(FIntVector4, SourceBufferDim)
			SHADER_PARAMETER(FIntVector, TargetBufferDim)
			SHADER_PARAMETER(int32, HalfOffsetSearchCount)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionSeparateSourceTarget : SHADER_PERMUTATION_BOOL("NONLOCALMEAN_SEPARATE_SOURCE");
		class FDimensionTargetWeightLayout : SHADER_PERMUTATION_ENUM_CLASS("NLM_WEIGHTLAYOUT", ENonLocalMeanWeightLayout);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionSeparateSourceTarget, FDimensionTargetWeightLayout>;
	};

	//--------------------------------------------------------------------------------------------------------------------
	// Collaborative filtering shaders
	//	1. Tiling
	// 
	// Copy textures to buffer based on copy channel config
	// Buffer layout
	// |Pixel 1                          | Pixel 2|...|Pixel n|
	//  tex1.rgb|tex2.rgb|tex3.rgb
	class FCopyTextureToBufferCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FCopyTextureToBufferCS)
		SHADER_USE_PARAMETER_STRUCT(FCopyTextureToBufferCS, FGlobalShader)
	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Source)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Dest)
			SHADER_PARAMETER(FIntPoint, TextureSize)
			SHADER_PARAMETER(int32, CopyChannelCount)
			SHADER_PARAMETER(int32, CopyChannelOffset)
			SHADER_PARAMETER(int32, BufferChannelOffset)
			SHADER_PARAMETER(int32, BufferChannelSize)
			SHADER_PARAMETER(FIntRect, CopyRegion)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}
		
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		static EPixelFormat GetDestFloatFormat(uint32 BufferBytesPerElement)
		{
			// Dest buffer can be float32 or float16
			EPixelFormat BufferFloatFormat = PF_R32_FLOAT;
			if (BufferBytesPerElement == sizeof(int16_t))
			{
				BufferFloatFormat = PF_R16F;
			}
			return BufferFloatFormat;
		}

		static const int kMaxSourceChannelCount = 4;

		class FDimensionSourceChannelCount : SHADER_PERMUTATION_RANGE_INT("SOURCE_CHANNEL_COUNT", 1, kMaxSourceChannelCount);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionSourceChannelCount>;
	};

	class FNormalizeTextureCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FNormalizeTextureCS)
		SHADER_USE_PARAMETER_STRUCT(FNormalizeTextureCS, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWSource)
			SHADER_PARAMETER(FIntPoint, TextureSize)
		END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};


	//==============================================================
	// Regression utility shaders
	// Given, X, Y, W buffer, calculate
	// A = X^T W X
	// B = X^T W Y
	// for each pixel footage, where 
	//		A has a dimension of |F|x|F|
	//		B has a dimension of |F|x|3|, |F| is the dimension of the feature count
	// typically it would be, 7 (1 constant, 3 albedo, 3 normal), TODO: add normalized depth.
	// 1. Calculate weighted quadratic form A, and the generalized weighted multiplication B
	// 2. Use solver to get reconstruction weight A^{-1} B (Ax = B, A=7x7, B=7x3)
	// 3. A1 Multiplication of X B and write to pixels for each image. 
	//	  A2. Gather temporal pixels based on weights
	// or
	// 3. B1 Multiplication of X B and gather at the same time.
	namespace RegressionKernel
	{
		FRDGBufferRef AllocateMatrixfBuffer(FRDGBuilder& GraphBuilder, int32 NumOfMatrices, int32 Dim0, int32 Dim1, const TCHAR* Name);

		enum class EWeightedMultiplicationType : uint8
		{
			Quadratic,		// X^T W X
			Generalized,	// X^T W Y
			MAX
		};

		static const TCHAR* GetEventName(EWeightedMultiplicationType WeightedMultiplicationType)
		{
			static const TCHAR* const kEventNames[] = {
				TEXT("X^TWX"),
				TEXT("X^TWY"),
			};
			static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EWeightedMultiplicationType::MAX), "Fix me");
			return kEventNames[int32(WeightedMultiplicationType)];
		}

		// Calculate
		// X^T Diag(W) Y, where Y might be equal to X
		// Note that the data is gathered in place to calculate the matrix
		// for each pixel. This avoids data duplication saved to memory at the cost of increasing memory bandwidth utilization
		// it is useful when memory size is small.
		class FInPlaceBatchedMatrixMultiplicationCS : public FGlobalShader
		{
			DECLARE_GLOBAL_SHADER(FInPlaceBatchedMatrixMultiplicationCS);
			SHADER_USE_PARAMETER_STRUCT(FInPlaceBatchedMatrixMultiplicationCS, FGlobalShader);

		public:
			static FRDGBufferRef AllocateResultBuffer(FRDGBuilder& GraphBuilder, FIntPoint Size, int F, int A);

			BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, X)
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer       , W)
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Y)
				SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Result)

				// Parameters for matrix multiplication
				SHADER_PARAMETER(FIntPoint, XDim) // NxF
				SHADER_PARAMETER(int32, WDim)	  // N	
				SHADER_PARAMETER(FIntPoint, YDim) // NxA

				// Parameters for the batched data storage for the image
				SHADER_PARAMETER(FIntPoint, TextureSize)
				SHADER_PARAMETER(int32, PatchDistance)
				SHADER_PARAMETER(int32, NumOfTemporalFrames)
				SHADER_PARAMETER(int32, NumOfWeigthsPerPixelPerFrame)

				// Used to accelerate the performance while maintaining good quality.
				SHADER_PARAMETER(int32, SamplingStep)

				SHADER_PARAMETER(int32, SourceFrameIndex) // which frame is currently to be denoised

			END_SHADER_PARAMETER_STRUCT()


			static int GetThreadGroupSize()
			{
				return NON_LOCAL_MEAN_THREAD_GROUP_SIZE;
			}
			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
			{
				FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
				OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), GetThreadGroupSize());
			}

			static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
			{
				return ShouldCompileNFORShadersForProject(Parameters.Platform);
			}

			static EPixelFormat GetXYFloatFormat(uint32 BufferBytesPerElement)
			{
				// Buffer can be float32 or float16
				EPixelFormat BufferFloatFormat = PF_R32_FLOAT;
				if (BufferBytesPerElement == sizeof(int16_t))
				{
					BufferFloatFormat = PF_R16F;
				}
				return BufferFloatFormat;
			}

			static const int kMaxSourceChannelCount = 4;

			class FDimWeightedMultiplicationType : SHADER_PERMUTATION_ENUM_CLASS("WEIGHTED_MULTIPLICATION_TYPE", EWeightedMultiplicationType);
			class FDimAddConstantFeatureDim : SHADER_PERMUTATION_BOOL("APPEND_CONSTANT_DIMENSION_TO_X");
			class FDimNumFeature : SHADER_PERMUTATION_RANGE_INT("NUM_FEATURE", 6, 3);
			class FDimOptimizeTargetMatrixMultiplication : SHADER_PERMUTATION_BOOL("SMALL_MATRIX_OPTIMIZE");
			class FDimUseSamplingStep : SHADER_PERMUTATION_BOOL("USE_SAMPLING_STEP");
			class FDimensionWeightLayout : SHADER_PERMUTATION_ENUM_CLASS("NLM_WEIGHTLAYOUT", ENonLocalMeanWeightLayout);
			using FPermutationDomain = TShaderPermutationDomain<FDimWeightedMultiplicationType, FDimAddConstantFeatureDim, FDimOptimizeTargetMatrixMultiplication, 
				FDimNumFeature, FDimUseSamplingStep, FDimensionWeightLayout>;
		};


		class FLinearSolverCS : public FGlobalShader
		{
			DECLARE_GLOBAL_SHADER(FLinearSolverCS);
			SHADER_USE_PARAMETER_STRUCT(FLinearSolverCS, FGlobalShader);

		public:
			BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, A)
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, B)
				SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Result)

				SHADER_PARAMETER(FIntPoint, ADim) // FxF
				SHADER_PARAMETER(FIntPoint, BDim) // FxA

				SHADER_PARAMETER(int, NumOfElements)
				SHADER_PARAMETER(int, NumOfElementsPerRow)

				SHADER_PARAMETER(float, Lambda)
				SHADER_PARAMETER(float, MinLambda)
				SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWSuccessAndFailIndexBuffer)
			END_SHADER_PARAMETER_STRUCT()

			enum class ESolverType : uint32
			{
				NewtonSchulz,
				Cholesky,
				NewtonCholesky,  
				MAX
			};

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
			{
				FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
				OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			}

			static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
			{
				return ShouldCompileNFORShadersForProject(Parameters.Platform);
			}

			class FDimNumFeature : SHADER_PERMUTATION_RANGE_INT("NUM_FEATURE", 6, 3);
			class FDimSolverType : SHADER_PERMUTATION_ENUM_CLASS("LINEAR_SOLVER_TYPE", ESolverType);
			class FDimOutputIndices : SHADER_PERMUTATION_BOOL("OUTPUT_INDICES");
			using FPermutationDomain = TShaderPermutationDomain<FDimNumFeature, FDimSolverType, FDimOutputIndices>;
		};

		enum class EInputMatrixType : uint32
		{
			Success,
			Fail,
			MAX
		};

		class FLinearSolverBuildIndirectDispatchArgsCS : public FGlobalShader
		{
			DECLARE_GLOBAL_SHADER(FLinearSolverBuildIndirectDispatchArgsCS);
			SHADER_USE_PARAMETER_STRUCT(FLinearSolverBuildIndirectDispatchArgsCS, FGlobalShader);
		public:

			BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SuccessAndFailIndexBuffer)
				SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsBuffer)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
			{
				FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
				OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			}

			static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
			{
				return ShouldCompileNFORShadersForProject(Parameters.Platform);
			}

			class FDimInputMatrixType : SHADER_PERMUTATION_ENUM_CLASS("INPUT_MATRIX_TYPE", EInputMatrixType);
			using FPermutationDomain = TShaderPermutationDomain <FDimInputMatrixType>;
		};

		class FLinearSolverIndirectCS : public FGlobalShader
		{
			DECLARE_GLOBAL_SHADER(FLinearSolverIndirectCS);
			SHADER_USE_PARAMETER_STRUCT(FLinearSolverIndirectCS, FGlobalShader);

		public:
			BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
				SHADER_PARAMETER_STRUCT_INCLUDE(FLinearSolverCS::FParameters, CommonParameters)
				RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SuccessAndFailIndexBuffer)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
			{
				FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
				OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			}

			static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
			{
				return ShouldCompileNFORShadersForProject(Parameters.Platform);
			}

			class FDimNumFeature : SHADER_PERMUTATION_RANGE_INT("NUM_FEATURE", 6, 3);
			class FDimSolverType : SHADER_PERMUTATION_ENUM_CLASS("LINEAR_SOLVER_TYPE", FLinearSolverCS::ESolverType);
			class FDimInputMatrixType : SHADER_PERMUTATION_ENUM_CLASS("INPUT_MATRIX_TYPE", EInputMatrixType);
			class FDimOutputIndices : SHADER_PERMUTATION_BOOL("OUTPUT_INDICES");
			using FPermutationDomain = TShaderPermutationDomain<FDimNumFeature, FDimSolverType, FDimInputMatrixType, FDimOutputIndices>;
		};

		// reconstruct with the weights into an image
		// Given X(|N|x|F|) and B (|F|x|3|)
		// spatial only (temporal frame, T = 1), Result = X * B, accumulate on each pixel to get shaper result
		// spatial temporal (T =3,5).
		//	Result_{pi} = X_i * w * B {i=T/2} + \sum_{i!=T/2}\sum_{j \in patch_i} (w * X_j * B)

		class FReconstructSpatialTemporalImage : public FGlobalShader
		{
			DECLARE_GLOBAL_SHADER(FReconstructSpatialTemporalImage);
			SHADER_USE_PARAMETER_STRUCT(FReconstructSpatialTemporalImage, FGlobalShader);

		public:
			BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, X)
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer       , W)
				SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, B)
				SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWReconstruction)
				SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint4>, RWReconstructBuffer)
				SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<UlongType>, RWReconstructBuffer64)


				SHADER_PARAMETER(FIntPoint, XDim) // N(=PxT) x F
				SHADER_PARAMETER(int32, WDim)	  // N	
				SHADER_PARAMETER(FIntPoint, BDim) // Px3

				// Parameters for the batched data storage for the image
				SHADER_PARAMETER(FIntPoint, TextureSize)
				SHADER_PARAMETER(int32, PatchDistance)
				SHADER_PARAMETER(int32, FrameIndex)

				SHADER_PARAMETER(int32, NumOfTemporalFrames)
				SHADER_PARAMETER(int32, NumOfWeigthsPerPixelPerFrame)

				SHADER_PARAMETER(float, AlbedoOffset)
			END_SHADER_PARAMETER_STRUCT()


			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
			{
				FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
				OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
				OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
			}

			static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
			{
				return ShouldCompileNFORShadersForProject(Parameters.Platform);
			}

			static EPixelFormat GetXFloatFormat(uint32 BufferBytesPerElement)
			{
				// Buffer can be float32 or float16
				EPixelFormat BufferFloatFormat = PF_R32_FLOAT;
				if (BufferBytesPerElement == sizeof(int16_t))
				{
					BufferFloatFormat = PF_R16F;
				}
				return BufferFloatFormat;
			}

			enum class EReconstructionType : uint8
			{
				Scatter,
				Gather,
				MAX
			};

			static const TCHAR* GetEventName(EReconstructionType ReconstructionType)
			{
				static const TCHAR* const kEventNames[] = {
					TEXT("Scatter"),
					TEXT("Gather"),
				};
				static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EWeightedMultiplicationType::MAX), "Fix me");
				return kEventNames[int32(ReconstructionType)];
			}

			class FDimReconstructionType : SHADER_PERMUTATION_ENUM_CLASS("RECONSTRUCTION_TYPE", EReconstructionType);
			class FDimPreAlbedoDivide : SHADER_PERMUTATION_ENUM_CLASS("PRE_ALBEDO_DIVIDE", EAlbedoDivideRecoverPhase);
			class FDimNumFeature : SHADER_PERMUTATION_RANGE_INT("NUM_FEATURE", 6, 3);
			class FDimensionWeightLayout : SHADER_PERMUTATION_ENUM_CLASS("NLM_WEIGHTLAYOUT", ENonLocalMeanWeightLayout);
			using FPermutationDomain = TShaderPermutationDomain<FDimReconstructionType, FDimPreAlbedoDivide, FDimNumFeature, FDimensionWeightLayout>;
		};
	}

	class FAccumulateBufferToTextureCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FAccumulateBufferToTextureCS);
		SHADER_USE_PARAMETER_STRUCT(FAccumulateBufferToTextureCS, FGlobalShader);

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, StructuredBufferSource)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<UlongType>, ReconstructBuffer64)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWTarget)

			SHADER_PARAMETER(FIntPoint, TextureSize)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimPreAlbedoDivide : SHADER_PERMUTATION_ENUM_CLASS("PRE_ALBEDO_DIVIDE", EAlbedoDivideRecoverPhase);
		using FPermutationDomain = TShaderPermutationDomain<FDimPreAlbedoDivide>;
	};


	//--------------------------------------------------------------------------------------------------------------------
	// Bandwidth selection

	class FMSEEstimationCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FMSEEstimationCS)
			SHADER_USE_PARAMETER_STRUCT(FMSEEstimationCS, FGlobalShader)

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Variance)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, Image)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, FilteredImage)
			SHADER_PARAMETER(FIntPoint, TextureSize)
			SHADER_PARAMETER(int32, VarianceChannelOffset)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, MSE)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
			OutEnvironment.SetDefine(TEXT("SOURCE_CHANNEL_COUNT"), 4);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}

		class FDimensionVarianceType : SHADER_PERMUTATION_ENUM_CLASS("IMAGE_VARIANCE_TYPE", EVarianceType);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionVarianceType>;
	};

	class FGenerateSelectionMapCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FGenerateSelectionMapCS)
			SHADER_USE_PARAMETER_STRUCT(FGenerateSelectionMapCS, FGlobalShader)

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, FilteredMSEs, [2])
			SHADER_PARAMETER(FIntPoint, TextureSize)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWSelectionMap)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	class FCombineFilteredImageCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FCombineFilteredImageCS)
			SHADER_USE_PARAMETER_STRUCT(FCombineFilteredImageCS, FGlobalShader)

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, FilteredImages, [2])
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SelectionMap)
			SHADER_PARAMETER(FIntPoint, TextureSize)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWFilteredImage)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NON_LOCAL_MEAN_THREAD_GROUP_SIZE);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return ShouldCompileNFORShadersForProject(Parameters.Platform);
		}
	};

	//--------------------------------------------------------------------------------------------------------------------
	// NFOR texture description and main denoising entry

	struct FNFORTextureDesc
	{
		FRDGTextureRef Image;

		/** the specific channels for this feature*/
		int32		  ChannelOffset;
		int32		  ChannelCount;

		/** The number of channels of the image*/
		int32		  NumOfChannel;

		FNFORTextureDesc(FRDGTextureRef Image, int32 ChannelOffset = 0, int32 ChannelCount = 4, int32 NumOfChannel = 4) :
			Image(Image), ChannelOffset(ChannelOffset), ChannelCount(ChannelCount), NumOfChannel(NumOfChannel) {}

	};

	struct FFeatureDesc
	{
		union
		{
			FNFORTextureDesc Feature;
			FNFORTextureDesc Data;
		};

		FNFORTextureDesc Variance;

		/** Variance type, greyscale, normal, or colored */
		EVarianceType VarianceType;

		/**Indicate no need to denoise if true**/
		bool bCleanFeature;

		FFeatureDesc(FNFORTextureDesc Feature, FNFORTextureDesc Variance, EVarianceType VarianceType = EVarianceType::GreyScale, bool bCleanFeature = false)
			:Feature(Feature), Variance(Variance), VarianceType(VarianceType), bCleanFeature(bCleanFeature) {}
	};

	typedef FFeatureDesc FRadianceDesc;

	// Denoise features based on non-local mean on its variance texture.
	void FilterFeatures(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const TArray<FFeatureDesc>& FeatureDescs);

	// Denoise the radiance based on spatial temporal features.
	bool FilterMain(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const TArray<FRadianceDesc>& Radiances,
		const TArray<FFeatureDesc>& FeatureDescs,
		const FRDGTextureRef& DenoisedRadiance);

}