// Copyright Epic Games, Inc. All Rights Reserved.

#include "NFORRegressionCPUSolver.h"
#include "NFORWeightedLSRCommon.h"

#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"

#include "HAL/IConsoleManager.h"
#include "Math/Quat.h"

// Just to be sure, also added this in Eigen.Build.cs
#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

namespace NFORRegressionSolverCPU
{
	TAutoConsoleVariable<int32> CVarNFORRegressionRecombineType(
		TEXT("r.NFOR.CPU.Regression.Recombine.Type"),
		0,
		TEXT("0: Apply patch reconstruction to patch for current frame, weighted sum of neighbor patch to the current pixel for other frames.")
		TEXT("1: Apply weighted sum of neighbor patch to the current pixel for all frames."),
		ECVF_RenderThreadSafe);

	enum class ERecombineType : int32
	{
		Auto,
		Gather,
		MAX
	};

	ERecombineType GetRecombineType()
	{
		const int32 RecombineType = FMath::Clamp(
			CVarNFORRegressionRecombineType.GetValueOnAnyThread(),
			static_cast<int32>(ERecombineType::Auto),
			static_cast<int32>(ERecombineType::MAX)-1);

		return static_cast<ERecombineType>(RecombineType);
	}

	struct NFORFirstOrderRegressionState
	{
		TArray<FLinearColor> Y; // WxHxC
		TArray<float>		 X; // WxHxF, F is the number of feature channel
		TArray<float>		 W; // WxHxN, N is the total number of weights over temporal frames = (2*PatchDistance+1)^2 * T

		TArray<FLinearColor> Buffer; // WxHxC

		FIntPoint CurrentSize = FIntPoint::ZeroValue;
		int CurrentNumChannels = 0;
		int CurrentNumWeights = 0;
		int CurrentNumRadianceChannels = 0;
		FIntPoint CurrentOffset = FIntPoint::ZeroValue;
		int NumOfFrames = 1;

		// Size: Texture image size Y
		void Update(FIntPoint TextureSize, int NumFeatureChannel, int NumWeights, int NumRadianceChannels);
		void Update(const FWeightedLSRDesc& WeightedLSRDesc);
	};

	void Apply(NFORFirstOrderRegressionState& FirstOrderRegressionState);

	struct FNFORLinearSolve
	{
		TArray<float> A; // N*FxF
		TArray<float> B; // N*FxC

		int CurrentNumElements = 0;
		int CurrentNumFeatureChannel = 0;
		int CurrentNumRadianceChannels = 0;

		void Update(int NumOfElements, int NumFeatureChannel, int NumRadianceChannels);
	};

	void Solve(FNFORLinearSolve& NFORLinearSolve);

	void NFORFirstOrderRegressionState::Update(FIntPoint Size, int NumFeatureChannel, int NumWeights, int NumRadianceChannels)
	{
		int NewSize = Size.X * Size.Y;
		const bool bImageSizeChanged = Y.Num() != NewSize;
		const bool bNumFeatureChannelChanged = CurrentNumChannels != NumFeatureChannel;
		const bool bNumWeightsChanged = CurrentNumWeights != NumWeights;

		if (bImageSizeChanged)
		{
			Y.SetNumUninitialized(NewSize);
			Buffer.SetNumUninitialized(NewSize);
		}

		if (bImageSizeChanged || bNumFeatureChannelChanged)
		{
			X.SetNumUninitialized(NewSize * NumFeatureChannel);
		}

		if (bImageSizeChanged || bNumWeightsChanged)
		{
			W.SetNumUninitialized(NewSize * NumWeights);
		}

		CurrentSize = Size;
		CurrentNumChannels = NumFeatureChannel;
		CurrentNumWeights = NumWeights;
		CurrentNumRadianceChannels = NumRadianceChannels;
	}

	template <typename MatrixType>
	void PrintEigenMatrix(const MatrixType& matrix) {
		for (int i = 0; i < matrix.rows(); ++i) {
			for (int j = 0; j < matrix.cols(); ++j) {
				UE_LOGF(LogTemp, Log, "%.5f ", static_cast<float>(matrix(i, j)));
			}
			UE_LOGF(LogTemp, Log, "\n");
		}
	}

	void NFORFirstOrderRegressionState::Update(const FWeightedLSRDesc& WeightedLSRDesc)
	{

		int NewSize = WeightedLSRDesc.TextureSize.X * WeightedLSRDesc.TextureSize.Y;
		const bool bImageSizeChanged = Y.Num() != (NewSize * WeightedLSRDesc.NumOfFrames);
		const bool bNumFeatureChannelChanged = CurrentNumChannels != WeightedLSRDesc.NumOfFeatureChannels;
		const bool bNumWeightsChanged = CurrentNumWeights != WeightedLSRDesc.NumOfWeightsPerPixel;
		const bool bOffsetChanged = CurrentOffset != WeightedLSRDesc.Offset;

		if (bImageSizeChanged)
		{
			Y.SetNumUninitialized(NewSize * WeightedLSRDesc.NumOfFrames);
			Buffer.SetNumUninitialized(NewSize);
		}

		if (bImageSizeChanged || bNumFeatureChannelChanged)
		{
			X.SetNumUninitialized(NewSize * WeightedLSRDesc.NumOfFeatureChannels);
		}


		int WeightSize = NewSize;
		if (WeightedLSRDesc.SolverType == EWeightedLSRSolverType::Tiled)
		{
			WeightSize = WeightedLSRDesc.Width * WeightedLSRDesc.Height;

		}

		if (bImageSizeChanged || bNumWeightsChanged)
		{
			W.SetNumUninitialized(WeightSize * WeightedLSRDesc.NumOfWeightsPerPixel);
		}

		CurrentSize = WeightedLSRDesc.TextureSize;
		CurrentNumChannels = WeightedLSRDesc.NumOfFeatureChannels;
		CurrentNumWeights = WeightedLSRDesc.NumOfWeightsPerPixel;
		CurrentNumRadianceChannels = WeightedLSRDesc.NumOfRadianceChannels;
		CurrentOffset = WeightedLSRDesc.Offset;
		NumOfFrames = WeightedLSRDesc.NumOfFrames;
	}

	FIntPoint ClampPointMirrored(FIntPoint P, FIntPoint Size)
	{
		FIntPoint SizeMax = Size - 1;

		int32 X = FMath::Abs(SizeMax.X - FMath::Abs(SizeMax.X - P.X));
		int32 Y = FMath::Abs(SizeMax.Y - FMath::Abs(SizeMax.Y - P.Y));

		return FIntPoint(X, Y);
	}

	void Apply(NFORFirstOrderRegressionState& FirstOrderRegressionState)
	{
		// Set up X, Y, W and solve with Eigen library
		// Input:
		// W = NonLocalMean weights
		// X = feature dimension
		// Y = Radiance

		// This function solves a weighted linear regression problem
		// Predicting a weight beta such that XB predicts the measured image value Y.
		// for each point
		// Solve Loss	= \sum wi (yi-xi^t\betai)^2
		//				= (Y-XB) W (Y-XB)
		// where:
		//	X = Xp or Xp-Xc
		// 
		// Solution:
		//  B = (X^T W X)^{-1} X^T W Y
		// Reconstruct = X B

		/*
		example code to solve the weighted linear regression for a patch

		int32 N = FirstOrderRegressionState.CurrentNumWeights;
		int32 D = FirstOrderRegressionState.CurrentNumChannels;
		Eigen::VectorXf W(N);
		Eigen::MatrixXf Y(N, 3);
		Eigen::MatrixXf X(N, D);

		Eigen::VectorXf WSqrt = W.cwiseSqrt();
		Eigen::VectorXf WDiag = WSqrt.asDiagonal();
		Eigen::MatrixXf Beta = (X.transpose() * WDiag * X).colPivHouseholderQr().solve(X.transpose() * WDiag * Y);
		Eigen::MatrixXf Reconstruct = X * Beta;

		call the code above for each pixel and write to the target buffer for value and weights accumulation. Each pixel is a weighted
		sum of NumWeights values.
		*/
		NFORFirstOrderRegressionState& Ctx = FirstOrderRegressionState;
		Ctx.Buffer.Init(FLinearColor::Transparent, Ctx.Y.Num());

		int32 N = Ctx.CurrentNumWeights;
		int32 D = Ctx.CurrentNumChannels;
		int32 B = Ctx.CurrentNumRadianceChannels;
		int32 T = Ctx.NumOfFrames;
		int32 R = (FMath::Sqrt(float(N / T)) - 1) / 2;
		int32 n = N / T;

		Eigen::VectorXf W = Eigen::VectorXf::Zero(N);
		Eigen::MatrixXf Y = Eigen::MatrixXf::Zero(N, 3);
		Eigen::MatrixXf X = Eigen::MatrixXf::Zero(N, D / T);

		int XPadding = Ctx.CurrentOffset.X;
		int YPadding = Ctx.CurrentOffset.Y;
		int32 SingleFrameSize = (Ctx.CurrentSize.X - 2 * XPadding) * (Ctx.CurrentSize.Y - 2 * XPadding);

		ERecombineType RecombineType = GetRecombineType();

		bool bPrintDebug = true;

		//TODO: Parallelize on GPU thread.
		for (int Py = YPadding; Py < Ctx.CurrentSize.Y - YPadding; ++Py)
		{
			for (int Px = XPadding; Px < Ctx.CurrentSize.X - XPadding; ++Px)
			{

				// For each pixel, solve the WLR.
				FIntPoint P = FIntPoint(Px, Py);
				int32 PIndex = (P.X + P.Y * Ctx.CurrentSize.X);

				{
					// The weight index is stored in the internal rectangular.
					FIntPoint WP = P - Ctx.CurrentOffset;

					int32 WPIndex = (WP.X + WP.Y * (Ctx.CurrentSize.X - 2 * XPadding));

					for (int i = 0; i < N; ++i)
					{
						int32 WOffset = i % n;
						int32 TOffset = i / n;
						int32 SerializedWPIndex = SingleFrameSize * n * TOffset + WPIndex * n + WOffset;

						W(i) = Ctx.W[SerializedWPIndex];
					}
				}

				// Fill the data.
				for (int Qy = Py - R; Qy <= Py + R; ++Qy)
				{
					for (int Qx = Px - R; Qx <= Px + R; ++Qx)
					{
						FIntPoint Q = FIntPoint(Qx, Qy);
						FIntPoint QMirrored = ClampPointMirrored(Q, Ctx.CurrentSize);
						int32 QIndex = (QMirrored.X + QMirrored.Y * Ctx.CurrentSize.X);


						FIntPoint LocalQ = Q - (P - FIntPoint(R, R));
						int LocalIndex = (LocalQ.X + LocalQ.Y * (2 * R + 1));

						for (int ti = 0; ti < T; ++ti)
						{
							float* YPtr = ((float*)Ctx.Y.GetData()) + (QIndex * T + ti) * B;
							Y(LocalIndex + ti * n, 0) = *YPtr;
							Y(LocalIndex + ti * n, 1) = *(YPtr + 1);
							Y(LocalIndex + ti * n, 2) = *(YPtr + 2);

						}


						for (int di = 0; di < D; ++di)
						{
							int FOffset = di % (D / T);
							int TOffset = di / (D / T);
							X(LocalIndex + TOffset * n, FOffset) = Ctx.X[QIndex * D + di];
						}
					}
				}

				// Solve for the whole patch.
				Eigen::VectorXf WSqrt = W.cwiseSqrt();
				Eigen::MatrixXf WDiag = WSqrt.asDiagonal();
				Eigen::MatrixXf AMatrix = X.transpose() * WDiag * X;
				Eigen::MatrixXf BMatrix = X.transpose() * WDiag * Y;
				Eigen::MatrixXf Beta = (AMatrix).colPivHouseholderQr().solve(BMatrix);
				Eigen::MatrixXf Reconstruct = X * Beta;

				// Write back: Two strategy:
				// One:
				//		1. Write back the current patch image to each pixel in the patch for the current frame
				//			Find the best fit for the patch given the similarity to the current center patch. 
				//			Each pixel is a weighted sum of denoised self based on all surrounding blocks.
				//		2. for history, gather all pixel contribution to the current center pixel.
				// Two:
				//		For all frames, gather all pixel contribution to the current pixel.

				for (int Qy = Py - R; Qy <= Py + R; ++Qy)
				{
					for (int Qx = Px - R; Qx <= Px + R; ++Qx)
					{
						FIntPoint Q = FIntPoint(Qx, Qy);
						FIntPoint QMirrored = ClampPointMirrored(Q, Ctx.CurrentSize);
						int32 QIndex = (QMirrored.X + QMirrored.Y * Ctx.CurrentSize.X);

						FIntPoint LocalQ = Q - (P - FIntPoint(R, R));
						int LocalIndex = (LocalQ.X + LocalQ.Y * (2 * R + 1));

						for (int i = 0; i < T; ++i)
						{
							int TemporalLocalIndex = LocalIndex + i * n;

							if (RecombineType == ERecombineType::Auto)
							{
								if (i == T / 2)
								{
									Ctx.Buffer[QIndex] += W(TemporalLocalIndex) *
										FLinearColor(Reconstruct(TemporalLocalIndex, 0), Reconstruct(TemporalLocalIndex, 1), Reconstruct(TemporalLocalIndex, 2), 1);
								}
								else
								{
									Ctx.Buffer[PIndex] += W(TemporalLocalIndex) *
										FLinearColor(Reconstruct(TemporalLocalIndex, 0), Reconstruct(TemporalLocalIndex, 1), Reconstruct(TemporalLocalIndex, 2), 1);
								}
							}
							else
							{
								Ctx.Buffer[PIndex] += W(TemporalLocalIndex) *
									FLinearColor(Reconstruct(TemporalLocalIndex, 0), Reconstruct(TemporalLocalIndex, 1), Reconstruct(TemporalLocalIndex, 2), 1);
							}
						}
					}
				} // Write back
			}
		}

		// Copy back from the buffer.
		for (int Py = 0; Py < Ctx.CurrentSize.Y; ++Py)
		{
			for (int Px = 0; Px < Ctx.CurrentSize.X; ++Px)
			{
				FIntPoint P = FIntPoint(Px, Py);
				int32 PIndex = (P.X + P.Y * Ctx.CurrentSize.X);

				Ctx.Y[PIndex] = Ctx.Buffer[PIndex];
			}
		}

	}

	void FNFORLinearSolve::Update(int NumOfElements, int NumFeatureChannel, int NumRadianceChannels)
	{
		if (CurrentNumElements != NumOfElements ||
			CurrentNumFeatureChannel != NumFeatureChannel)
		{
			CurrentNumElements = NumOfElements;
			CurrentNumFeatureChannel = NumFeatureChannel;

			A.SetNumUninitialized(CurrentNumElements * CurrentNumFeatureChannel * CurrentNumFeatureChannel);
		}

		if (CurrentNumElements != NumOfElements ||
			CurrentNumFeatureChannel != NumFeatureChannel ||
			CurrentNumRadianceChannels != NumRadianceChannels)
		{
			CurrentNumElements = NumOfElements;
			CurrentNumFeatureChannel = NumFeatureChannel;
			CurrentNumRadianceChannels = NumRadianceChannels;

			B.SetNumUninitialized(CurrentNumElements * CurrentNumFeatureChannel * CurrentNumRadianceChannels);
		}
	}

	void Solve(FNFORLinearSolve& NFORLinearSolve)
	{

		const int F = NFORLinearSolve.CurrentNumFeatureChannel;
		const int C = NFORLinearSolve.CurrentNumRadianceChannels;
		const int N = NFORLinearSolve.CurrentNumElements;

		const int ASize = F * F;
		const int BSize = F * C;

		Eigen::MatrixXf A = Eigen::MatrixXf::Zero(F, F);
		Eigen::MatrixXf B = Eigen::MatrixXf::Zero(F, C);

		for (int i = 0; i < N; ++i)
		{
			for (int j = 0; j < ASize; ++j)
			{
				A(j / F, j % F) = NFORLinearSolve.A[i * ASize + j];
			}

			for (int j = 0; j < BSize; ++j)
			{
				B(j / C, j % C) = NFORLinearSolve.B[i * BSize + j];
			}

			Eigen::MatrixXf Beta = (A).colPivHouseholderQr().solve(B);

			for (int j = 0; j < BSize; ++j)
			{
				NFORLinearSolve.B[i * BSize + j] = Beta(j / C, j % C);
			}
		}
	}
}

template <typename PixelType>
static void CopyTextureFromGPUToCPU(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, FIntPoint Size, TArray<PixelType>& DstArray)
{
	FRHIGPUTextureReadback Readback(TEXT("NFOR::TextureReadback"));
	Readback.EnqueueCopy(RHICmdList, SrcTexture, FIntVector::ZeroValue, 0, FIntVector(Size.X, Size.Y, 1));
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	int32_t SrcStride = 0;
	const PixelType* SrcBuffer = static_cast<PixelType*>(Readback.Lock(SrcStride, nullptr));

	PixelType* DstBuffer = DstArray.GetData();
	for (int Y = 0; Y < Size.Y; Y++, DstBuffer += Size.X, SrcBuffer += SrcStride)
	{
		FPlatformMemory::Memcpy(DstBuffer, SrcBuffer, Size.X * sizeof(PixelType));

	}
	Readback.Unlock();
}

template <typename PixelType>
static void CopyTextureFromCPUToGPU(FRHICommandListImmediate& RHICmdList, const TArray<PixelType>& SrcArray, FIntPoint Size, FRHITexture* DstTexture)
{
	uint32_t DestStride;
	FLinearColor* DstBuffer = static_cast<PixelType*>(RHICmdList.LockTexture2D(DstTexture, 0, RLM_WriteOnly, DestStride, false));
	DestStride /= sizeof(PixelType);
	const FLinearColor* SrcBuffer = SrcArray.GetData();
	for (int Y = 0; Y < Size.Y; Y++, SrcBuffer += Size.X, DstBuffer += DestStride)
	{
		FPlatformMemory::Memcpy(DstBuffer, SrcBuffer, Size.X * sizeof(PixelType));
	}
	RHICmdList.UnlockTexture2D(DstTexture, 0, false);
}

template <typename ElementType>
static void CopyBufferFromGPUToCPU(FRHICommandListImmediate& RHICmdList, FRHIBuffer* SrcBuffer, uint32 NumElement, TArray<ElementType>& DstArray)
{
	FRHIGPUBufferReadback Readback(TEXT("NFOR::BufferReadBack"));
	uint32_t NumBytes = NumElement * sizeof(ElementType);
	Readback.EnqueueCopy(RHICmdList, SrcBuffer, NumBytes);
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	const ElementType* SrcRawBuffer = static_cast<ElementType*>(Readback.Lock(NumBytes));

	ElementType* DstBuffer = DstArray.GetData();
	FPlatformMemory::Memcpy(DstBuffer, SrcRawBuffer, NumBytes);
	Readback.Unlock();
}

template <typename ElementType>
static void CopyBufferFromCPUToGPU(FRHICommandListImmediate& RHICmdList, TArray<ElementType>& SrcArray, uint32 NumElement, FRHIBuffer* DstBuffer)
{
	uint32_t NumBytes = NumElement * sizeof(ElementType);
	void* Data = RHICmdList.LockBuffer(DstBuffer, 0, NumBytes, RLM_WriteOnly);
	ElementType* DstRawBuffer = static_cast<ElementType*>(Data);

	const ElementType* SrcBuffer = SrcArray.GetData();
	FPlatformMemory::Memcpy(DstRawBuffer, SrcBuffer, NumBytes);
	RHICmdList.UnlockBuffer(DstBuffer);
}

// CPU solver relevants used for verification.	
BEGIN_SHADER_PARAMETER_STRUCT(FFirstOrderRegressionParameters, )
	RDG_BUFFER_ACCESS(X, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(Y, ERHIAccess::CopySrc)
	RDG_BUFFER_ACCESS(Ys, ERHIAccess::CopySrc)
	RDG_BUFFER_ACCESS(W, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(Reconstruct, ERHIAccess::CopyDest)
	SHADER_PARAMETER(int32, T)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FLinearSolverParameters, )
	RDG_BUFFER_ACCESS(A, ERHIAccess::CopySrc)
	RDG_BUFFER_ACCESS(B, ERHIAccess::CopySrc)
	RDG_BUFFER_ACCESS(X, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

static NFORRegressionSolverCPU::NFORFirstOrderRegressionState RegressionState;
static NFORRegressionSolverCPU::FNFORLinearSolve NFORLinearSolveState;

void SolveWeightedLSRCPU(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FRDGBufferRef& Feature,
	const FRDGTextureRef& Radiance,
	const FRDGBufferRef& NonLocalMeanWeightsBuffer,
	const FRDGTextureRef& FilteredRadiance,
	const FWeightedLSRDesc& WeightedLSRDesc,
	const FRDGBufferRef Radiances,
	const FRDGTextureRef& SourceAlbedo)
{
	FIntPoint TextureSize = Radiance->Desc.Extent;
	FFirstOrderRegressionParameters* FirstOrderRegressionParameters = GraphBuilder.AllocParameters<FFirstOrderRegressionParameters>();

	{
		FirstOrderRegressionParameters->X = Feature;
		FirstOrderRegressionParameters->Y = Radiance;
		FirstOrderRegressionParameters->Ys = Radiances;
		FirstOrderRegressionParameters->W = NonLocalMeanWeightsBuffer;
		FirstOrderRegressionParameters->Reconstruct = FilteredRadiance;
	}

	// Need to read GPU mask outside Pass function, as the value is not refreshed inside the pass
	GraphBuilder.AddPass(RDG_EVENT_NAME("SolvWeightedLSR (Eigen CPU Sequential)"), FirstOrderRegressionParameters, ERDGPassFlags::Readback,
		[FirstOrderRegressionParameters, WeightedLSRDesc, TextureSize](FRHICommandListImmediate& RHICmdList)
		{
			RegressionState.Update(WeightedLSRDesc);

			// Read W,Y and X to CPU	
			CopyBufferFromGPUToCPU(RHICmdList, FirstOrderRegressionParameters->Ys->GetRHI(),
				FirstOrderRegressionParameters->Ys->GetSize() / FirstOrderRegressionParameters->Ys->GetStride(), RegressionState.Y);
			CopyBufferFromGPUToCPU(RHICmdList, FirstOrderRegressionParameters->X->GetRHI(),
				FirstOrderRegressionParameters->X->GetSize() / FirstOrderRegressionParameters->X->GetStride(), RegressionState.X);
			CopyBufferFromGPUToCPU(RHICmdList, FirstOrderRegressionParameters->W->GetRHI(),
				FirstOrderRegressionParameters->W->GetSize() / FirstOrderRegressionParameters->W->GetStride(), RegressionState.W);

			// Run the linear regression
			NFORRegressionSolverCPU::Apply(RegressionState);

			// Copy back
			CopyTextureFromCPUToGPU(RHICmdList, RegressionState.Y, TextureSize, FirstOrderRegressionParameters->Reconstruct->GetRHI()->GetTexture2D());
		});
}

void SolveLinearEquationCPU(
	FRDGBuilder& GraphBuilder, 
	const FRDGBufferRef& A, 
	const FRDGBufferRef& B,
	const int32 NumOfElements,
	const FIntPoint BDim,
	const FRDGBufferRef& X)
{
	FLinearSolverParameters* LinearSolverParameters = GraphBuilder.AllocParameters<FLinearSolverParameters>();
	LinearSolverParameters->A = A;
	LinearSolverParameters->B = B;
	LinearSolverParameters->X = X;

	GraphBuilder.AddPass(RDG_EVENT_NAME("ReconstructWeights (Eigen CPU Sequential)"), LinearSolverParameters, ERDGPassFlags::Readback,
		[LinearSolverParameters, NumOfElements, BDim](FRHICommandListImmediate& RHICmdList)
		{
			NFORLinearSolveState.Update(NumOfElements, BDim.X, BDim.Y);

			// Copy W,Y and X to CPU			
			CopyBufferFromGPUToCPU(RHICmdList, LinearSolverParameters->A->GetRHI(),
				LinearSolverParameters->A->GetSize() / LinearSolverParameters->A->GetStride(), NFORLinearSolveState.A);

			CopyBufferFromGPUToCPU(RHICmdList, LinearSolverParameters->B->GetRHI(),
				LinearSolverParameters->B->GetSize() / LinearSolverParameters->B->GetStride(), NFORLinearSolveState.B);

			NFORRegressionSolverCPU::Solve(NFORLinearSolveState);

			// Copy back
			CopyBufferFromCPUToGPU(RHICmdList, NFORLinearSolveState.B,
				LinearSolverParameters->B->GetSize() / LinearSolverParameters->B->GetStride(), LinearSolverParameters->X->GetRHI());
		});
}
