// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "Math/MathFwd.h"

struct FWeightedLSRDesc;
class FSceneView;

void SolveWeightedLSRCPU(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FRDGBufferRef& Feature,
	const FRDGTextureRef& Radiance,
	const FRDGBufferRef& NonLocalMeanWeightsBuffer,
	const FRDGTextureRef& FilteredRadiance,
	const FWeightedLSRDesc& WeightedLSRDesc,
	const FRDGBufferRef Radiances,
	const FRDGTextureRef& SourceAlbedo);

// Solve AX=B
void SolveLinearEquationCPU(
	FRDGBuilder& GraphBuilder,
	const FRDGBufferRef& A,
	const FRDGBufferRef& B,
	const int32 NumOfElements,
	const FIntPoint BDim,
	const FRDGBufferRef& X);