// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntPoint.h"
#include "Math/MathFwd.h"

enum class EWeightedLSRSolverType : uint8
{
	Full,			// whole image input
	Tiled			// tiled image input, Overlapping region data are included
};

struct FWeightedLSRDesc
{
	int32 NumOfFeatureChannels; // F
	int32 NumOfFeatureChannelsPerFrame;
	int32 NumOfWeightsPerPixel; // N
	int32 NumOfWeightsPerPixelPerFrame;
	int32 NumOfRadianceChannels;
	int32 NumOfRadianceChannelsPerFrame;
	int32 Width;				// W
	int32 Height;				// H

	FIntPoint TileStartPosition;	// The start position of the tile 
	FIntPoint Offset;				// Used in tiled mode as we need to copy data outside of the region

	int32 NumOfFrames;			// T
	FIntPoint TextureSize;		// (W+2B,H+2B) where B is the border size
	EWeightedLSRSolverType SolverType;
};
